[CoreRedirects]
; AnimNextWarping plugin -> UAFWarping plugin
+PackageRedirects=(OldName="/Script/AnimNextWarping", NewName="/Script/UAFWarping")

+StructRedirects=(OldName="/Script/AnimNextWarping.SteeringTraitSharedData", NewName="/Script/UAFWarping.SteeringTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextWarping.AnimNextSteeringTask", NewName="/Script/UAFWarping.AnimNextSteeringTask")
+StructRedirects=(OldName="/Script/AnimNextWarping.StrafeWarpFootData", NewName="/Script/UAFWarping.StrafeWarpFootData")
+StructRedirects=(OldName="/Script/AnimNextWarping.StrafeWarpingTraitSharedData", NewName="/Script/UAFWarping.StrafeWarpingTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextWarping.AnimNextStrafeWarpingTask", NewName="/Script/UAFWarping.AnimNextStrafeWarpingTask")
+StructRedirects=(OldName="/Script/AnimNextWarping.WarpTestTraitSharedData", NewName="/Script/UAFWarping.WarpTestTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextWarping.AnimNextWarpTestTask", NewName="/Script/UAFWarping.AnimNextWarpTestTask")