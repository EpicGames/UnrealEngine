// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFStrafeWarpingNode.h"
#include "Animation/AnimRootMotionProvider.h"
#include "AnimNextWarpingLog.h"
#include "TwoBoneIK.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/ValueRuntime/Transformers/BoneSpace.h"
#include "VisualLogger/VisualLogger.h"

namespace UE::UAF
{

#if ENABLE_ANIM_DEBUG
bool bAnimNextStrafeWarpingNodeEnabled = true;
static FAutoConsoleVariableRef CVarAnimNextStrafeWarpingNodeEnabled(
	TEXT("a.AnimNext.StrafeWarpingNode.Enabled"),
	bAnimNextStrafeWarpingNodeEnabled,
	TEXT("True will enable strafe warping for AnimNext. Equivalent to setting alpha to non-zero.")
);
#else
constexpr bool bAnimNextStrafeWarpingNodeEnabled = true;
#endif

struct FUAFStrafeWarpingUtils
{
	static FVector GetAxisVector(const EAxis::Type& InAxis)
	{
		switch (InAxis)
		{
		case EAxis::X:
			return FVector::ForwardVector;
		case EAxis::Y:
			return FVector::RightVector;
		default:
			return FVector::UpVector;
		};
	}

	static float SignedAngleRadBetweenNormals(const FVector& From, const FVector& To, const FVector& Axis)
	{
		const float FromDotTo = FVector::DotProduct(From, To);
		const float Angle = FMath::Acos(FromDotTo);
		const FVector Cross = FVector::CrossProduct(From, To);
		const float Dot = FVector::DotProduct(Cross, Axis);
		return Dot >= 0 ? Angle : -Angle;
	}

	static bool IsBoneChildOf(FAttributeSetIndex ChildIndex, FAttributeSetIndex ParentIndex, FAttributeTypedSetPtr BoneSet)
	{
		if (!BoneSet || !ParentIndex.IsValid())
		{
			return false;
		}

		FAttributeSetIndex CurrentIndex = ChildIndex;
		while (CurrentIndex.IsValid())
		{
			if (CurrentIndex == ParentIndex)
			{
				return true;
			}

			// Keep looking
			CurrentIndex = BoneSet->GetParentIndex(CurrentIndex);
		}

		return false;
	}
};

///////////////////////////////////////////////////////////
// FUAFStrafeWarpingNode
void FUAFStrafeWarpingPostAnimOp::DoWarpRootMotion(FUAFAnimOpValueEvaluator& Evaluator, const FVector& RotationAxisVector, const FVector& TargetMoveDir, float& OutTargetOrientationAngleRad)
{
	check(SharedData);

	const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

	const USkeleton* Skeleton = AnimOpEvaluationContext.GetSkeleton();
	FPoseValueBundleCoWRef* InputRef = Evaluator.GetEvaluationStack().PeekMutable();
	FPoseValueBundle& ValueBundle = InputRef->GetMutable();

	const FAttributeNamedSetPtr NamedSet = ValueBundle.GetNamedSet();
	const FAttributeTypedSetPtr AttributeTypedSet = NamedSet->FindTypedSet<FTransformAnimationAttribute>();
	if (!AttributeTypedSet)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FUAFStrafeWarpingPostAnimOp::DoWarpRootMotion, AttributeTypedSet not found");
		return;
	}

	const FAttributeSetIndex SetIndex = AttributeTypedSet->FindIndex(UE::Anim::IAnimRootMotionProvider::AttributeName);
	if (!SetIndex)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FUAFStrafeWarpingPostAnimOp::DoWarpRootMotion, SetIndex for IAnimRootMotionProvider not found");
		return;
	}

	// This attribute is present within the current named set
	const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FTransformAnimationAttribute>();
	TBoundValueMap<FTransformAnimationAttribute>* Map = ValueBundle.GetBoundValueMaps().Find<FTransformAnimationAttribute>(MappingKey);

	if (!Map)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FUAFStrafeWarpingPostAnimOp::DoWarpRootMotion, This attribute exists within our named set but isn't present in the output value bundle");
		return;
	}

	FTransform& ThisFrameRootMotionTransform = (*Map)[SetIndex].Value;

	FVector RootMotionDeltaTranslation = ThisFrameRootMotionTransform.GetTranslation();
	const FQuat PreviousRootMotionDeltaRotation = RootMotionDeltaRotation;
	RootMotionDeltaRotation = ThisFrameRootMotionTransform.GetRotation();

	const float RootMotionDeltaSpeed = RootMotionDeltaTranslation.Size() / DeltaTime;
	if (RootMotionDeltaSpeed < SharedData->MinRootMotionSpeedThreshold)
	{
		// If we're under the threshold, snap orientation angle to 0, and let interpolation handle the delta
		OutTargetOrientationAngleRad = 0.0f;
	}
	else
	{
		const FVector PreviousRootMotionDeltaDirection = RootMotionDeltaDirection;
		// Hold previous direction if we can't calculate it from current move delta, because the root is no longer moving
		RootMotionDeltaDirection = RootMotionDeltaTranslation.GetSafeNormal(UE_SMALL_NUMBER, PreviousRootMotionDeltaDirection);
		OutTargetOrientationAngleRad = FUAFStrafeWarpingUtils::SignedAngleRadBetweenNormals(
			RootMotionDeltaDirection, TargetMoveDir, RotationAxisVector);

		// Motion Matching may return an animation that deviates a lot from the movement direction (e.g movement direction going bwd and motion matching could return the fwd animation for a few frames)
		// When that happens, since we use the delta between root motion and movement direction, we would be over-rotating the lower body and breaking the pose during those frames
		// So, when that happens we use the inverse of the root motion direction to calculate our target rotation. 
		// This feels a bit 'hacky' but its the only option I've found so far to mitigate the problem
		if (SharedData->LocomotionAngleDeltaThreshold > 0.f)
		{
			if (FMath::Abs(FMath::RadiansToDegrees(OutTargetOrientationAngleRad)) > SharedData->LocomotionAngleDeltaThreshold)
			{
				OutTargetOrientationAngleRad = FMath::UnwindRadians(OutTargetOrientationAngleRad + FMath::DegreesToRadians(180.0f));
				RootMotionDeltaDirection = -RootMotionDeltaDirection;
			}
		}
		/* No prediction in first iteration for AnimNext. This code is copy-pasta from AnimNode_OrientationWarping
		// If there is translation in predicted root motion, use it if the orientation error is less than current error
		if (TargetTime > 0 && CurrentAnimAsset && !PredictedRootMotionDeltaTranslation.IsNearlyZero(UE_SMALL_NUMBER))
		{
			PredictedRootMotionDeltaTranslation.Normalize();
			float PredictedOrientationErrorAngleRad = FUAFStrafeWarpingUtils::SignedAngleRadBetweenNormals(
				PredictedRootMotionDeltaTranslation, LocomotionForward, RotationAxisVector);

			// The future orientation will often match the current, so add a small delta to avoid further testing for same values
			if (FMath::Abs(PredictedOrientationErrorAngleRad) + UE_KINDA_SMALL_NUMBER < FMath::Abs(TargetOrientationAngleRad))
			{
				// Note: Don't update root motion direction, as the root motion direction is what we are playing,
				// not the future. Updating root motion direction will break counter compensate.
				// Also, we rely on the built in interp for continuity / smoothness
#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG
				FutureRootMotionDeltaDirection = PredictedRootMotionDeltaTranslation;
				bUsedFutureRootMotion = true;
#endif
				TargetOrientationAngleRad = PredictedOrientationErrorAngleRad;
			}
		}
		*/

		// Don't compensate interpolation by the root motion angle delta if the previous direction isn't valid.
		if (SharedData->bCounterCompenstateInterpolationByRootMotion && !PreviousRootMotionDeltaDirection.IsNearlyZero(UE_SMALL_NUMBER))
		{
			float RootMotionDeltaAngleRad = 0.0f;
			// Counter the interpolated orientation angle by the root motion direction angle delta.
			// This prevents our interpolation from fighting the natural root motion that's flowing through the graph.
			// To correctly measure the amount to counter, we need to unrotate our previous delta direction by our previous rotation
			// As the previous direction delta is relative to the previous rotation delta
			RootMotionDeltaAngleRad = FUAFStrafeWarpingUtils::SignedAngleRadBetweenNormals(
				RootMotionDeltaDirection, PreviousRootMotionDeltaRotation.UnrotateVector(PreviousRootMotionDeltaDirection),
				RotationAxisVector);

			// Root motion may have large deltas i.e. bad blends or sudden direction changes like pivots.
			// If there's an instantaneous pop in root motion direction, this is likely a pivot.
			const float MaxRootMotionDeltaToCompensateRad = FMath::DegreesToRadians(SharedData->MaxRootMotionDeltaToCompensateDegrees);
			if (FMath::Abs(RootMotionDeltaAngleRad) < MaxRootMotionDeltaToCompensateRad)
			{
				CounterCompensateTargetAngleRad += RootMotionDeltaAngleRad;
				float CounterCompensateAngle = FMath::FInterpTo(0, CounterCompensateTargetAngleRad, DeltaTime,
					SharedData->CounterCompensateInterpSpeed);
				OrientationAngleForPoseWarpRad = FMath::UnwindRadians(OrientationAngleForPoseWarpRad + CounterCompensateAngle);
				CounterCompensateTargetAngleRad -= CounterCompensateAngle;
			}
		}

		// Rotate the root motion delta fully by the warped angle
		const FVector WarpedRootMotionTranslationDelta = FQuat(RotationAxisVector, OutTargetOrientationAngleRad).RotateVector(
			RootMotionDeltaTranslation);
		ThisFrameRootMotionTransform.SetTranslation(WarpedRootMotionTranslationDelta);
	}

#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
	if (FVisualLogger::IsRecording())
	{
		constexpr float DebugDrawScale = 1.f;
		const FTransform ComponentTransform = RootBoneTransform;

		FVector DebugArrowOffset = FVector::ZAxisVector * DebugDrawScale;
		//uint8 DebugAlpha = AnimNodeOrientationWarpingDebugTransparency ? 255 * BlendWeight : 255;
		uint8 DebugAlpha = 255;
		FColor DebugColor = FColor::Green;

		// Draw debug shapes
		{
			const FVector ForwardDirection = ComponentTransform.GetRotation().RotateVector(TargetMoveDir);
			if (const UObject* VLogObject = HostObject.Pin().Get())
			{
				UE_VLOG_CIRCLE_THICK(VLogObject, "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset + ForwardDirection * 100.f * DebugDrawScale,
					FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
				UE_VLOG_ARROW(VLogObject, "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset,
					ComponentTransform.GetLocation() + DebugArrowOffset + ForwardDirection * 100.f * DebugDrawScale,
					FColor::Red.WithAlpha(DebugAlpha), TEXT(""));

				const FVector RotationDirection = ComponentTransform.GetRotation().RotateVector(RootMotionDeltaDirection);

				DebugArrowOffset += FVector::ZAxisVector * DebugDrawScale;

				UE_VLOG_CIRCLE_THICK(VLogObject, "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset + RotationDirection * 100.f * DebugDrawScale,
					FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
				UE_VLOG_ARROW(VLogObject, "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset,
					ComponentTransform.GetLocation() + DebugArrowOffset + RotationDirection * 100.f * DebugDrawScale,
					FColor::Blue.WithAlpha(DebugAlpha), TEXT(""));

				// Debug for predictive root motion, not currently implemented
				//if (bUsedFutureRootMotion && bGraphDrivenWarping)
				//{
				//	const FVector FutureRotationDirection = ComponentTransform.GetRotation().RotateVector(FutureRootMotionDeltaDirection);

				//	UE_VLOG_CIRCLE_THICK(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
				//		ComponentTransform.GetLocation() + DebugArrowOffset + FutureRotationDirection * 100.f * DebugDrawScale,
				//		FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
				//	UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
				//		ComponentTransform.GetLocation() + DebugArrowOffset,
				//		ComponentTransform.GetLocation() + DebugArrowOffset + FutureRotationDirection * 100.f * DebugDrawScale,
				//		FColor::Yellow.WithAlpha(DebugAlpha), TEXT(""));
				//}

				const float ActualOrientationAngleDegrees = FMath::RadiansToDegrees(OrientationAngleForPoseWarpRad);
				const FVector WarpedRotationDirection = RotationDirection.RotateAngleAxis(ActualOrientationAngleDegrees, RotationAxisVector);

				DebugArrowOffset += FVector::ZAxisVector * DebugDrawScale;

				UE_VLOG_CIRCLE_THICK(VLogObject, "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset + WarpedRotationDirection * 100.f * DebugDrawScale,
					FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
				UE_VLOG_ARROW(VLogObject, "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset,
					ComponentTransform.GetLocation() + DebugArrowOffset + WarpedRotationDirection * 100.f * DebugDrawScale,
					FColor::Green.WithAlpha(DebugAlpha), TEXT(""));
			}
		}
	}
#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
}

void FUAFStrafeWarpingPostAnimOp::InitializeSpineData(TArrayView<FSpineBoneData> OutSpineBoneData, const TArray<FName>& SpineBoneNames, FAttributeTypedSetPtr BoneSet)
{
	QUICK_SCOPE_CYCLE_COUNTER(FStrafeWarpingNode_InitializeSpineData);

	check(OutSpineBoneData.Num() == SpineBoneNames.Num());
		
	if (SpineBoneNames.Num() == 0)
	{
		return;
	}
		
	for (int32 i = 0; i < SpineBoneNames.Num(); i++)
	{
		OutSpineBoneData[i].Weight = 0.0f;
		OutSpineBoneData[i].BoneIndex = BoneSet->FindIndex(SpineBoneNames[i]);
	}

	// Calculate weight

	// Sort bones indices so we can transform parent before child
	OutSpineBoneData.Sort(FSpineBoneData::FCompareBoneIndex());

	// Assign Weights.
	TArray<int32, TInlineAllocator<20>> IndicesToUpdate;

	// Note reverse iteration
	for (int32 Index = OutSpineBoneData.Num() - 1; Index >= 0; Index--)
	{
		// If this bone's weight hasn't been updated, scan its parents.
		// If parents have weight, we add it to 'ExistingWeight'.
		// split (1.f - 'ExistingWeight') between all members of the chain that have no weight yet.
		if (OutSpineBoneData[Index].Weight == 0.f)
		{
			IndicesToUpdate.Reset(OutSpineBoneData.Num());
			float ExistingWeight = 0.f;
			IndicesToUpdate.Add(Index);

			for (int32 ParentIndex = Index - 1; ParentIndex >= 0; ParentIndex--)
			{
				if (FUAFStrafeWarpingUtils::IsBoneChildOf(OutSpineBoneData[Index].BoneIndex, OutSpineBoneData[ParentIndex].BoneIndex, BoneSet))
				{
					if (OutSpineBoneData[ParentIndex].Weight > 0.f)
					{
						ExistingWeight += OutSpineBoneData[ParentIndex].Weight;
					}
					else
					{
						IndicesToUpdate.Add(ParentIndex);
					}
				}
			}

			check(IndicesToUpdate.Num() > 0);
			const float WeightToShare = 1.f - ExistingWeight;
			const float IndividualWeight = WeightToShare / float(IndicesToUpdate.Num());

			for (int32 UpdateListIndex = 0; UpdateListIndex < IndicesToUpdate.Num(); UpdateListIndex++)
			{
				OutSpineBoneData[IndicesToUpdate[UpdateListIndex]].Weight = IndividualWeight;
			}
		}
	}
}

void FUAFStrafeWarpingPostAnimOp::DoWarpPose(FUAFAnimOpValueEvaluator& Evaluator, const FVector& RotationAxisVector)
{
	using namespace UE::UAF::Transformers;

	check(SharedData);

	const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

	const USkeleton* Skeleton = AnimOpEvaluationContext.GetSkeleton();
	FPoseValueBundleCoWRef* InputRef = Evaluator.GetEvaluationStack().PeekMutable();
	FPoseValueBundle& ValueBundle = InputRef->GetMutable();

	TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = ValueBundle.FindBoneTransforms();
	if (!BoneTransforms)
	{
		return;
	}

	FPoseValueBundleStack ComponentSpaceCollection(Evaluator.GetActiveNamedSet());
	ComponentSpaceCollection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

	// @todo: add a LocalToComponent evaluating transforms in a lazy way to avoid having to compute the entire pose in component space!
	FBoneSpace::LocalToComponent(ValueBundle, ComponentSpaceCollection);

	TBoundValueMap<FBoneTransformAnimationAttribute>* ComponentSpaceBoneTransforms = ComponentSpaceCollection.FindBoneTransforms();
	if (!ComponentSpaceBoneTransforms)
	{
		return;
	}

	FAttributeTypedSetPtr BoneSet = ComponentSpaceBoneTransforms->GetTypedSet();

	const float RootOffset = FMath::UnwindRadians(OrientationAngleForPoseWarpRad * SharedData->DistributedBoneOrientationAlpha);

	const float IKFootRootOrientationAlpha = 1.f - SharedData->DistributedBoneOrientationAlpha;

	// Rotate IK Foot Root
	if (!FMath::IsNearlyZero(IKFootRootOrientationAlpha, KINDA_SMALL_NUMBER))
	{
		{
			const FQuat IKRootBoneRotation = FQuat(RotationAxisVector, OrientationAngleForPoseWarpRad * IKFootRootOrientationAlpha);
			// IK Feet 
			// We want these to keep their original component space orientation
			// But we want them to translate based on some rotation
			const int32 NumIKFootBones = SharedData->FootData.Num();

			if (NumIKFootBones > 0)
			{
				for (int32 ArrayIndex = 0; ArrayIndex < NumIKFootBones; ArrayIndex++)
				{
					const FStrafeWarpFootData& FootData = SharedData->FootData[ArrayIndex];
					FAttributeSetIndex LegTipIndex = BoneSet->FindIndex(FootData.LegTip);
					if (!LegTipIndex.IsValid())
					{
						continue;
					}

					FTransform FootTargetTransform((*ComponentSpaceBoneTransforms)[LegTipIndex].Value.GetRotation(), (*ComponentSpaceBoneTransforms)[LegTipIndex].Value.GetTranslation());
					FootTargetTransform.SetLocation(IKRootBoneRotation.RotateVector(FootTargetTransform.GetLocation()));

#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
					if (FVisualLogger::IsRecording())
					{
						if (const UObject* VLogObject = HostObject.Pin().Get())
						{
							const float DebugDrawSpehereRadius = 10.f;
							UE_VLOG_SPHERE(VLogObject, "OrientationWarping", Display, RootBoneTransform.TransformPosition(FootTargetTransform.GetLocation()), DebugDrawSpehereRadius, FColor::Green, TEXT(""));
						}
					}
#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG

					if (SharedData->FootMode == EStrafeWarpingFootMode::DoBasicIK)
					{
						FAttributeSetIndex LegRootIndex = BoneSet->FindIndex(FootData.LegRoot);
						FAttributeSetIndex LegMidIndex = BoneSet->FindIndex(FootData.LegMid);

						// Validate data
						if (!LegRootIndex.IsValid() || !LegMidIndex.IsValid())
						{
							continue;
						}

						FTransform LegRootTransformCS((*ComponentSpaceBoneTransforms)[LegRootIndex].Value.GetRotation(), (*ComponentSpaceBoneTransforms)[LegRootIndex].Value.GetTranslation());
						FTransform LegMidTransformCS((*ComponentSpaceBoneTransforms)[LegMidIndex].Value.GetRotation(), (*ComponentSpaceBoneTransforms)[LegMidIndex].Value.GetTranslation());
						FTransform LegTipTransformCS((*ComponentSpaceBoneTransforms)[LegTipIndex].Value.GetRotation(), (*ComponentSpaceBoneTransforms)[LegTipIndex].Value.GetTranslation());

						// Calculate pole vector
						// By using the side direction of the knee and the target, we can ensure the pole target is always 90 degrees from the target direction
						FVector PoleTarget = FootTargetTransform.GetLocation() - LegRootTransformCS.GetLocation();
						FVector SideVectorCS = LegMidTransformCS.GetRotation().RotateVector(FootData.JointSideVector);
						PoleTarget = PoleTarget.Cross(SideVectorCS);

						// Pole vector is expected to be provided as a location in AnimationCore::SolveTwoBoneIK
						// so convert from direction to location
						PoleTarget += LegRootTransformCS.GetLocation();

						AnimationCore::SolveTwoBoneIK(LegRootTransformCS,
							LegMidTransformCS,
							LegTipTransformCS,
							PoleTarget,
							FootTargetTransform.GetLocation(),
							false,
							0.0,
							0.0
						);

						LegTipTransformCS.SetRotation(FootTargetTransform.GetRotation());

						// Apply results

						// Be careful reverse TTransform vs FQuat order of operations
						FTransform LocalTip = LegTipTransformCS * LegMidTransformCS.Inverse();
						FTransform LocalMid = LegMidTransformCS * LegRootTransformCS.Inverse();

						// Figure out local root by applying difference in CS. We can't use the parent component space since it hasn't been updated (unlike the foot and knee)
						// This avoids computing all the component space transforms again
						// We can set the local transform of the root by applying the difference between the component space root transform
						FQuat LocalRootRotationDiff = (*ComponentSpaceBoneTransforms)[LegRootIndex].Value.GetRotation().Inverse() * LegRootTransformCS.GetRotation();

						(*BoneTransforms)[LegTipIndex].Value.SetTranslation(LocalTip.GetLocation());
						(*BoneTransforms)[LegTipIndex].Value.SetRotation(LocalTip.GetRotation());
						(*BoneTransforms)[LegMidIndex].Value.SetTranslation(LocalMid.GetLocation());
						(*BoneTransforms)[LegMidIndex].Value.SetRotation(LocalMid.GetRotation());
						(*BoneTransforms)[LegRootIndex].Value.SetRotation((*BoneTransforms)[LegRootIndex].Value.GetRotation() * LocalRootRotationDiff);


#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
						if (FVisualLogger::IsRecording())
						{
							if (const UObject* VLogObject = HostObject.Pin().Get())
							{
								const float DebugDrawScale = 10.0f;
								const uint16 DebugDrawThickness = 1;
								const float DebugDrawSpehereRadius = 10.f;
								// Careful: FTransform multiplication is reveresed from quat!
									
								FTransform FootWorldTransformOriginal = FTransform((*ComponentSpaceBoneTransforms)[LegTipIndex].Value.GetRotation(), (*ComponentSpaceBoneTransforms)[LegTipIndex].Value.GetTranslation()) * RootBoneTransform;
								FTransform FootWorldTransformTarget = FTransform((*ComponentSpaceBoneTransforms)[LegTipIndex].Value.GetRotation(), FootTargetTransform.GetLocation()) * RootBoneTransform;
								FTransform FootWorldTransformSolved = LegTipTransformCS * RootBoneTransform;
								UE_VLOG_COORDINATESYSTEM(VLogObject, "OrientationWarping", Display, FootWorldTransformTarget.GetLocation(), FootWorldTransformTarget.GetRotation().Rotator(), DebugDrawScale, FColor::Green, DebugDrawThickness, TEXT(""));
								UE_VLOG_COORDINATESYSTEM(VLogObject, "OrientationWarping", Display, FootWorldTransformOriginal.GetLocation(), FootWorldTransformOriginal.GetRotation().Rotator(), DebugDrawScale, FColor::Green, DebugDrawThickness, TEXT(""));
								UE_VLOG_ARROW(VLogObject, "OrientationWarping", Display, FootWorldTransformOriginal.GetLocation(), FootWorldTransformTarget.GetLocation(), FColor::Magenta, TEXT(""));
								UE_VLOG_ARROW(VLogObject, "OrientationWarping", Display, LegMidTransformCS.GetLocation() + RootBoneTransform.GetLocation(), PoleTarget + RootBoneTransform.GetLocation(), FColor::Red, TEXT(""));
								UE_VLOG_SPHERE(VLogObject, "OrientationWarping", Display, FootWorldTransformSolved.GetTranslation(), DebugDrawSpehereRadius, FColor::Yellow, TEXT(""));
							}
						}
#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
					}


					// Set the IK bone to match the foot target
					// Better to do this even if we also did basic IK, in case other nodes later down the chain rely on the ik target bone

					FAttributeSetIndex IKTargetIndex = BoneSet->FindIndex(FootData.IKTargetBone);

					if (IKTargetIndex.IsValid())
					{
						// Set IK target to equal leg tip in component space
						FAttributeSetIndex IKTargetParent = BoneSet->GetParentIndex(IKTargetIndex);
						if (IKTargetParent.IsValid())
						{
							FTransform IKTargetCS = FootTargetTransform;
							FTransform IKTargetParentCS = (*ComponentSpaceBoneTransforms)[IKTargetParent].Value;
							FTransform IKTargetLocal = IKTargetCS * IKTargetParentCS.Inverse();

							(*BoneTransforms)[IKTargetIndex].Value = IKTargetLocal;
						}
					}

				}
			}
		}
	}

	// Rotate Root Bone first, as that cheaply rotates the whole pose with one transformation.
	// We do this with the pose in local space since we want it to propagate
	if (!FMath::IsNearlyZero(RootOffset, KINDA_SMALL_NUMBER))
	{
		FAttributeSetIndex RootBoneIndex(0);
		if (SharedData->PreserveOriginalRootRotation)
		{
			// Find all children of the root and adjust them
			for (FAttributeSetIndex BoneIndex(1); BoneIndex < BoneSet->Num(); ++BoneIndex)
			{
				if (BoneSet->GetParentIndex(BoneIndex) == RootBoneIndex)
				{
					// Is a child of the root
					const FVector LocalRotationVector = (*ComponentSpaceBoneTransforms)[BoneIndex].Value.GetRotation().UnrotateVector(RotationAxisVector);
					const FQuat RootRotation = FQuat(LocalRotationVector, RootOffset);
					(*BoneTransforms)[BoneIndex].Value.SetRotation((*BoneTransforms)[BoneIndex].Value.GetRotation() * RootRotation);
				}
			}
		}
		else
		{
			const FQuat RootRotation = FQuat(RotationAxisVector, RootOffset);
			FQuat RootBoneRotation = (*BoneTransforms)[RootBoneIndex].Value.GetRotation() * RootRotation;
			RootBoneRotation.Normalize();
			(*BoneTransforms)[RootBoneIndex].Value.SetRotation(RootBoneRotation);
		}
	}

	const int32 NumSpineBones = SharedData->SpineBones.Num();
	const bool bSpineOrientationAlpha = !FMath::IsNearlyZero(SharedData->DistributedBoneOrientationAlpha, KINDA_SMALL_NUMBER);
	const bool bUpdateSpineBones = (NumSpineBones > 0) && bSpineOrientationAlpha;

	if (bUpdateSpineBones)
	{
		// Todo: Can we get away with lazy init here? Does the ref pose skeleton change at runtime?
		// Todo: Cache spine bone data
		TArray<FSpineBoneData, TInlineAllocator<16>> SpineBoneDataArray;
		SpineBoneDataArray.SetNumUninitialized(NumSpineBones);
			
		InitializeSpineData(SpineBoneDataArray, SharedData->SpineBones, BoneSet);

		// Spine bones counter rotate body orientation evenly across all bones.
		// Note: reverse iteration is important! We go from child to parent
		for (int32 ArrayIndex = NumSpineBones - 1; ArrayIndex >= 0; ArrayIndex--)
		{
			const FSpineBoneData& BoneData = SpineBoneDataArray[ArrayIndex];
			if (!BoneData.BoneIndex.IsValid() || BoneData.Weight == 0.0f)
			{
				continue;
			}

			// Important note! The root was moved in local space, so our component transform array is actually out of date
			// However since we know everything rotated around RotationAxisVector, it doesn't matter for this calculation
			const FVector LocalRotationVector = (*ComponentSpaceBoneTransforms)[BoneData.BoneIndex].Value.GetRotation().UnrotateVector(RotationAxisVector);
			const FQuat SpineBoneCounterRotation = FQuat(LocalRotationVector, -OrientationAngleForPoseWarpRad * SharedData->DistributedBoneOrientationAlpha * BoneData.Weight);

			FQuat SpineBoneRotation = (*BoneTransforms)[BoneData.BoneIndex].Value.GetRotation() * SpineBoneCounterRotation;
			SpineBoneRotation.Normalize();

			(*BoneTransforms)[BoneData.BoneIndex].Value.SetRotation(SpineBoneRotation);
		}
	}
}

FUAFStrafeWarpingPostAnimOp::FUAFStrafeWarpingPostAnimOp()
: FUAFAnimOp(1)
{
	InitializeAs<FUAFStrafeWarpingPostAnimOp>();
}

void FUAFStrafeWarpingPostAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
{
	if (!SharedData || DeltaTime == 0.f || Alpha == 0.f || !bAnimNextStrafeWarpingNodeEnabled)
	{
		return;
	}

	{
		const FPoseValueBundleCoWRef* ValueBundle = Evaluator.GetEvaluationStack().Peek();
		check(ValueBundle != nullptr);

		if (ValueBundle->Get().IsEmpty())
		{
			return;
		}
	}

	float TargetOrientationAngleRad = 0.0f;
	const FVector RotationAxisVector = FUAFStrafeWarpingUtils::GetAxisVector(SharedData->RotationAxis);

	// Target Orientation is in world space, transform to root
	FQuat TargetRotation = RootBoneTransform.GetRotation().Inverse() * TargetOrientation; 
	FVector TargetMoveDir = TargetRotation.GetForwardVector();

	// Flatten locomotion direction, along the rotation axis.
	TargetMoveDir = (TargetMoveDir - RotationAxisVector.Dot(TargetMoveDir) * RotationAxisVector).GetSafeNormal();

	DoWarpRootMotion(Evaluator, RotationAxisVector, TargetMoveDir, TargetOrientationAngleRad);

	// Calculate the orientation warp angle for pose adjustments (spine and foot IK)
	const float MaxAngleCorrectionRad = FMath::DegreesToRadians(SharedData->MaxCorrectionDegrees);
	
	// Optionally interpolate the effective orientation towards the target orientation angle
	// When the orientation warping node becomes relevant, the input pose orientation may not be aligned with the desired orientation.
	// Instead of interpolating this difference, snap to the desired orientation if it's our first update to minimize corrections over-time.
	if ((SharedData->RotationInterpSpeed > 0.f) /*&& !bIsFirstUpdate */)
	{
		const float SmoothOrientationAngleRad = FMath::FInterpTo(OrientationAngleForPoseWarpRad, TargetOrientationAngleRad, DeltaTime, SharedData->RotationInterpSpeed);
		// Limit our interpolation rate to prevent pops.
		// @TODO: Use better, more physically accurate interpolation here.
		OrientationAngleForPoseWarpRad = FMath::Clamp(SmoothOrientationAngleRad, OrientationAngleForPoseWarpRad - MaxAngleCorrectionRad, OrientationAngleForPoseWarpRad + MaxAngleCorrectionRad);
	}
	else
	{
		OrientationAngleForPoseWarpRad = TargetOrientationAngleRad;
	}

	OrientationAngleForPoseWarpRad = FMath::Clamp(OrientationAngleForPoseWarpRad, -MaxAngleCorrectionRad, MaxAngleCorrectionRad);
	// Allow the alpha value of the node to affect the final rotation
	OrientationAngleForPoseWarpRad *= Alpha;

	if (FMath::IsNearlyZero(OrientationAngleForPoseWarpRad, KINDA_SMALL_NUMBER))
	{
		// No strafe angle, early out before hitting the pose modification code
		return;
	}

	DoWarpPose(Evaluator, RotationAxisVector);
}

///////////////////////////////////////////////////////////
// FUAFStrafeWarpingNode
FUAFStrafeWarpingNode::FUAFStrafeWarpingNode(FUAFAnimGraphUpdateContext& Context, const FUAFStrafeWarpingData& Data)
	: FUAFModifierAnimNode(Context)
{
	InitializeAs<FUAFStrafeWarpingNode>(Context);
	InitializeModifier(Context, Data);

	if (HasChild())
	{
		PostAnimOp.HostObject = Context.GetHostObject();
		PostAnimOp.SetDebugOwner(this);
		PostAnimOp.SharedData = &Data;

		SetPostAnimOp(&PostAnimOp);
	}
}

void FUAFStrafeWarpingNode::PreUpdate(FUAFAnimGraphUpdateContext& Context)
{
	if (PostAnimOp.SharedData)
	{
		PostAnimOp.DeltaTime = Context.GetDeltaTime();
		PostAnimOp.RootBoneTransform = PostAnimOp.SharedData->RootBoneTransform.GetValue(Context.GetVariablesOwner());
		PostAnimOp.TargetOrientation = PostAnimOp.SharedData->TargetOrientation.GetValue(Context.GetVariablesOwner());
		PostAnimOp.Alpha = PostAnimOp.SharedData->Alpha.GetValue(Context.GetVariablesOwner());
	}
}

#if UAF_TRACE_ENABLED
FString FUAFStrafeWarpingNode::GetDebugName() const
{
	static FString Name("Strafe Warping");
	return Name;
}

UStruct* FUAFStrafeWarpingNode::GetDebugStruct() const
{
	return FUAFStrafeWarpingData::StaticStruct();
}
#endif

FUAFAnimNodePtr FUAFStrafeWarpingData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
{
	return MakeAnimNode<FUAFStrafeWarpingNode>(Context, *this);
}

} // namespace UE::UAF
