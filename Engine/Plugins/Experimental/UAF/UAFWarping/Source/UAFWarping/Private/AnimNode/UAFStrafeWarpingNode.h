// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BindableValue/UAFBindableTypes.h"
#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNode.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNodeData.h"
#include "UAF/Attributes/AttributeTypedSet.h"

// @todo: remove this include and move common data into a separate header
#include "../StrafeWarpingTrait.h"

#include "UAFStrafeWarpingNode.generated.h"

namespace UE::UAF
{
struct FUAFStrafeWarpingData;

USTRUCT()
struct FUAFStrafeWarpingPostAnimOp : public FUAFAnimOp
{
	GENERATED_BODY()
	UAF_DECLARE_ANIMOP(FUAFStrafeWarpingPostAnimOp)

	FUAFStrafeWarpingPostAnimOp();

	virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;


	// Internal structure with some precomputed weight values and bone indices
	struct FSpineBoneData
	{
		FAttributeSetIndex BoneIndex;
		float Weight = 0.f;

		// Comparison Operator for Sorting
		struct FCompareBoneIndex
		{
			FORCEINLINE bool operator()(const FSpineBoneData& A, const FSpineBoneData& B) const
			{
				return A.BoneIndex < B.BoneIndex;
			}
		};
	};

	void DoWarpRootMotion(FUAFAnimOpValueEvaluator& Evaluator, const FVector& RotationAxisVector, const FVector& TargetMoveDir, float& OutTargetOrientationAngleRad);
	void DoWarpPose(FUAFAnimOpValueEvaluator& Evaluator, const FVector& RotationAxisVector);
	void InitializeSpineData(TArrayView<FSpineBoneData> OutSpineBoneData, const TArray<FName>& SpineBoneNames, FAttributeTypedSetPtr BoneSet);

	TWeakObjectPtr<const UObject> HostObject;

	/** Target orientation per instance */
	FQuat TargetOrientation = FQuat::Identity;
		
	/** Last root bone transform sampled */
	FTransform RootBoneTransform = FTransform::Identity;

	/** Delta in seconds between updates, populated during PreUpdate */
	float DeltaTime = 0.f;

	float Alpha = 1.0f;

	// Internal current frame root motion delta direction
	FVector RootMotionDeltaDirection = FVector::ZeroVector;

	// Internal current frame root motion delta angle
	FQuat RootMotionDeltaRotation = FQuat::Identity;

	// Target for counter compenstate, we keep the target so we can smoothly interp.
	float CounterCompensateTargetAngleRad = 0.0f;

	// Internal orientation warping angle
	float OrientationAngleForPoseWarpRad = 0.f;

	const FUAFStrafeWarpingData* SharedData = nullptr;
};

USTRUCT(DisplayName = "Strafe Warping")
struct FUAFStrafeWarpingData : public FUAFModifierAnimNodeData
{
	GENERATED_BODY()

	// Current strength of the skeletal control
	UPROPERTY(EditAnywhere, Category = Alpha)
	FBindableFloat Alpha = 1.0f;

	// The Orientation to steer towards
	UPROPERTY(EditAnywhere, Category = Evaluation)
	FBindableQuat TargetOrientation;
	
	// @TODO Temp / try to remove this. Shouldn't have to feed as argument
	// Last root bone transform sampled
	UPROPERTY(EditAnywhere, Category = Evaluation)
	FBindableTransform RootBoneTransform;

	// Rotation axis used when rotating the character body
	UPROPERTY(EditAnywhere, Category=Settings)
	TEnumAsByte<EAxis::Type> RotationAxis = EAxis::Z;

	// Specifies how much rotation is applied to the character body versus IK feet
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0", ClampMax="1.0"))
	float DistributedBoneOrientationAlpha = 0.5f;

	// Specifies the interpolation speed (in Alpha per second) towards reaching the final warped rotation angle
	// A value of 0 will cause instantaneous rotation, while a greater value will introduce smoothing
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0"))
	float RotationInterpSpeed = 10.f;

	// Same as RotationInterpSpeed, but for CounterCompensate smoothing. A value of 0 sample raw root motion.
	// Used to avoid stuttering from resampling root deltas. Root motion is already smooth, so a large value is our default (~75% of 60 fps).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0"))
	float CounterCompensateInterpSpeed = 45.f;

	// Max correction we're allowed to do per-second when using interpolation.
	// This minimizes pops when we have a large difference between current and target orientation.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0", EditCondition="RotationInterpSpeed > 0.0f"))
	float MaxCorrectionDegrees = 180.f;

	// Don't compensate our interpolator when the instantaneous root motion delta is higher than this. This is likely a pivot.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0", EditCondition="RotationInterpSpeed > 0.0f"))
	float MaxRootMotionDeltaToCompensateDegrees = 45.f;

	// Whether to counter compensate interpolation by the animated root motion angle change over time.
	// This helps to conserve the motion from our animation.
	// Disable this if your root motion is expected to be jittery, and you want orientation warping to smooth it out.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(EditCondition="RotationInterpSpeed > 0.0f"))
	bool bCounterCompenstateInterpolationByRootMotion = true;
	
	// Minimum root motion speed required to apply orientation warping
	// This is useful to prevent unnatural re-orientation when the animation has a portion with no root motion (i.e starts/stops/idles)
	// When this value is greater than 0, it's recommended to enable interpolation with RotationInterpSpeed > 0
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (ClampMin = "0.0"))
	float MinRootMotionSpeedThreshold = 10.0f;
	
	// Specifies an angle threshold to prevent erroneous over-rotation of the character, disabled with a value of 0
	//
	// When the effective orientation warping angle is detected to be greater than this value (default: 90 degrees) the locomotion direction will be inverted prior to warping
	// This will be used in the following equation: [Orientation = RotationBetween(RootMotionDirection, -LocomotionDirection)]
	//
	// Example: Playing a forward running animation while the motion is going backward 
	// Rather than orientation warping by 180 degrees, the system will warp by 0 degrees 
	UPROPERTY(EditAnywhere, Category=Evaluation, meta=(ClampMin="0.0", ClampMax="180.0"))
	float LocomotionAngleDeltaThreshold = 90.f;

	// When true, propagates any modification on the root bone down to all of its children
	// When false, will directly modify the root
	UPROPERTY(EditAnywhere, Category=Evaluation)
	bool PreserveOriginalRootRotation = true;

	// Spine bone definitions
	// Used to counter rotate the body in order to keep the character facing forward
	// The amount of counter rotation applied is driven by DistributedBoneOrientationAlpha
	// TODO: we ideally would use an equivlane to FBoneReference
	UPROPERTY(EditAnywhere, Category=Settings)
	TArray<FName> SpineBones;

	UPROPERTY(EditAnywhere, Category=Settings, meta=(DisplayName="IK Foot Bones"))
	TArray<FStrafeWarpFootData> FootData;

	UPROPERTY(EditAnywhere, Category=Settings)
	EStrafeWarpingFootMode FootMode = EStrafeWarpingFootMode::DoBasicIK;

	// FUAFAnimNodeData impl
	virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
};

// This node modifies an animation pose to orientate towards a desired move direction 
struct FUAFStrafeWarpingNode : FUAFModifierAnimNode
{
	FUAFStrafeWarpingNode(FUAFAnimGraphUpdateContext& Context, const FUAFStrafeWarpingData& InData);

	virtual void PreUpdate(FUAFAnimGraphUpdateContext& Context) override;

#if UAF_TRACE_ENABLED
	virtual FString GetDebugName() const override;
	virtual UStruct* GetDebugStruct() const override;
#endif
private:
	FUAFStrafeWarpingPostAnimOp PostAnimOp;
};

} // namespace UE::UAF