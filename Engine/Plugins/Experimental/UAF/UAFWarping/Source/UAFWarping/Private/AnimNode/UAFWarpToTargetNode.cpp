// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFWarpToTargetNode.h"
#include "Animation/AnimRootMotionProvider.h"
#include "AnimNextWarpingLog.h"
#include "TwoBoneIK.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/ValueRuntime/Transformers/BoneSpace.h"
#include "VisualLogger/VisualLogger.h"

namespace UE::UAF
{

FUAFWarpToTargetPostAnimOp::FUAFWarpToTargetPostAnimOp()
: FUAFAnimOp(1)
{
	InitializeAs<FUAFWarpToTargetPostAnimOp>();
}

void FUAFWarpToTargetPostAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
{
	const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

	const USkeleton* Skeleton = AnimOpEvaluationContext.GetSkeleton();
	FPoseValueBundleCoWRef* InputRef = Evaluator.GetEvaluationStack().PeekMutable();
	FPoseValueBundle& ValueBundle = InputRef->GetMutable();

	if (ValueBundle.IsEmpty())
	{
		return;
	}

	const FAttributeNamedSetPtr NamedSet = ValueBundle.GetNamedSet();
	if (!NamedSet)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FUAFWarpToTargetPostAnimOp::EvaluateValues, NamedSet not found");
		return;
	}

	const FAttributeTypedSetPtr AttributeTypedSet = NamedSet->FindTypedSet<FTransformAnimationAttribute>();
	if (!AttributeTypedSet)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FUAFWarpToTargetPostAnimOp::EvaluateValues, AttributeTypedSet not found");
		return;
	}

	const FAttributeSetIndex SetIndex = AttributeTypedSet->FindIndex(UE::Anim::IAnimRootMotionProvider::AttributeName);
	if (!SetIndex)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FUAFWarpToTargetPostAnimOp::EvaluateValues, SetIndex for IAnimRootMotionProvider not found");
		return;
	}

	// This attribute is present within the current named set
	const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FTransformAnimationAttribute>();
	TBoundValueMap<FTransformAnimationAttribute>* Map = ValueBundle.GetBoundValueMaps().Find<FTransformAnimationAttribute>(MappingKey);

	if (!Map)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FUAFWarpToTargetPostAnimOp::EvaluateValues, This attribute exists within our named set but isn't present in the output value bundle");
		return;
	}

	FTransform& RootMotionTransform = (*Map)[SetIndex].Value;
	RootMotionTransform = TargetRootBoneTransform.GetRelativeTransform(RootBoneTransform);
	
#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
	if (FVisualLogger::IsRecording())
	{
		if (const UObject* VLogObject = HostObject.Pin().Get())
		{
			UE_VLOG_COORDINATESYSTEM(VLogObject, "WarpToTarget", Display, RootBoneTransform.GetLocation(), RootBoneTransform.GetRotation().Rotator(), 10.f, FColor::Red, 1, TEXT(""));
			UE_VLOG_COORDINATESYSTEM(VLogObject, "WarpToTarget", Display, TargetRootBoneTransform.GetLocation(), TargetRootBoneTransform.GetRotation().Rotator(), 50.f, FColor::Green, 1, TEXT(""));
		}
	}
#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
}

///////////////////////////////////////////////////////////
// FUAFWarpToTargetNode
FUAFWarpToTargetNode::FUAFWarpToTargetNode(FUAFAnimGraphUpdateContext& Context, const FUAFWarpToTargetData& InData)
	: FUAFModifierAnimNode(Context)
{
	InitializeAs<FUAFWarpToTargetNode>(Context);
	InitializeModifier(Context, InData);

	Data = &InData;

	if (HasChild())
	{
		PostAnimOp.HostObject = Context.GetHostObject();
		PostAnimOp.SetDebugOwner(this);

		SetPostAnimOp(&PostAnimOp);
	}
}

void FUAFWarpToTargetNode::PreUpdate(FUAFAnimGraphUpdateContext& Context)
{
	check(Data);
	PostAnimOp.RootBoneTransform = Data->RootBoneTransform.GetValue(Context.GetVariablesOwner());
	PostAnimOp.TargetRootBoneTransform = Data->TargetRootBoneTransform.GetValue(Context.GetVariablesOwner());
}

#if UAF_TRACE_ENABLED
FString FUAFWarpToTargetNode::GetDebugName() const
{
	static FString Name("Warp To Target");
	return Name;
}

UStruct* FUAFWarpToTargetNode::GetDebugStruct() const
{
	return FUAFWarpToTargetData::StaticStruct();
}
#endif

FUAFAnimNodePtr FUAFWarpToTargetData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
{
	return MakeAnimNode<FUAFWarpToTargetNode>(Context, *this);
}

} // namespace UE::UAF
