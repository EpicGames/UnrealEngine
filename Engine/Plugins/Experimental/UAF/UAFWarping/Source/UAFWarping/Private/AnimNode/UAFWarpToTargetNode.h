// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BindableValue/UAFBindableTypes.h"
#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNode.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNodeData.h"
#include "UAF/Attributes/AttributeTypedSet.h"

#include "UAFWarpToTargetNode.generated.h"

namespace UE::UAF
{

USTRUCT()
struct FUAFWarpToTargetPostAnimOp : public FUAFAnimOp
{
	GENERATED_BODY()
	UAF_DECLARE_ANIMOP(FUAFWarpToTargetPostAnimOp)

	FUAFWarpToTargetPostAnimOp();

	virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	TWeakObjectPtr<const UObject> HostObject;

	FTransform RootBoneTransform = FTransform::Identity;
	FTransform TargetRootBoneTransform = FTransform::Identity;
};

USTRUCT(DisplayName = "Warp To Target")
struct FUAFWarpToTargetData : public FUAFModifierAnimNodeData
{
	GENERATED_BODY()

	// @TODO Temp / try to remove this. Shouldn't have to feed as argument
	// Last root bone transform sampled
	UPROPERTY(EditAnywhere, Category = Evaluation)
	FBindableTransform RootBoneTransform;

	UPROPERTY(EditAnywhere, Category = Evaluation)
	FBindableTransform TargetRootBoneTransform;

	// FUAFAnimNodeData impl
	virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
};

struct FUAFWarpToTargetNode : FUAFModifierAnimNode
{
	FUAFWarpToTargetNode(FUAFAnimGraphUpdateContext& Context, const FUAFWarpToTargetData& InData);

	virtual void PreUpdate(FUAFAnimGraphUpdateContext& Context) override;

#if UAF_TRACE_ENABLED
	virtual FString GetDebugName() const override;
	virtual UStruct* GetDebugStruct() const override;
#endif
private:
	FUAFWarpToTargetPostAnimOp PostAnimOp;
	const FUAFWarpToTargetData* Data = nullptr;
};

} // namespace UE::UAF