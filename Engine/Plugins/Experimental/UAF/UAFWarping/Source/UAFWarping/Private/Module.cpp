// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Module/AnimNextModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RigVMCore/RigVMRegistry.h"
#include "Animation/AnimSequence.h"
#include "StrafeWarpingTrait.h"

#include "AnimNextWarpingLog.h"
#include "EvaluationVM/EvaluationVM.h"

DEFINE_LOG_CATEGORY(LogAnimNextWarping);

namespace UE::UAF::Warping
{

class FModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		static UScriptStruct* const AllowedStructTypes[] =
		{
			FStrafeWarpFootData::StaticStruct(),
		};

		FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
		RigVMRegistry.RegisterStructTypes(AllowedStructTypes);
		
		// @TODO: Register any pin types needed for warping here for RigVM, currently we have none
		//static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
		//{
		//	// { UMyRigVMParamClass::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
		//};

		//FRigVMRegistry::Get().RegisterObjectTypes(AllowedObjectTypes);
	}

	virtual void ShutdownModule() override
	{
	}

};

}

IMPLEMENT_MODULE(UE::UAF::Warping::FModule, UAFWarping)
