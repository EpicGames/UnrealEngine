// Copyright Epic Games, Inc. All Rights Reserved.

#include "OffsetRootBoneTrait.h"

#include "AnimNextWarpingLog.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Math/SpringMath.h"
#include "EvaluationVM/EvaluationVM.h"
#include "VisualLogger/VisualLogger.h"

#if ENABLE_ANIM_DEBUG
namespace UE::UAF
{
	TAutoConsoleVariable<int32> CVarAnimNodeOffsetRootBoneEnable(TEXT("a.UAF.OffsetRootBone.Enable"), 1, TEXT("Toggle Offset Root Bone"));
}
#endif

namespace UE::UAF::OffsetRootBone
{

static bool ShouldExtractRootMotion(const EUAFOffsetRootBoneMode OffsetMode)
{
	switch (OffsetMode)
	{
	case EUAFOffsetRootBoneMode::Accumulate:
	case EUAFOffsetRootBoneMode::Interpolate:
		return true;
	case EUAFOffsetRootBoneMode::Release:
	default:
		return false;
	}
}

static bool ShouldCounterComponentDelta(const EUAFOffsetRootBoneMode OffsetMode)
{
	switch (OffsetMode)
	{
	case EUAFOffsetRootBoneMode::Accumulate:
	case EUAFOffsetRootBoneMode::Interpolate:
		return false;
	case EUAFOffsetRootBoneMode::Release:
	default:
		return true;
	}
}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FOffsetRootBoneTrait

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FOffsetRootBoneTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \

	// Trait implementation boilerplate
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FOffsetRootBoneTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FOffsetRootBoneTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);
		
		InstanceData->DeltaTime = TraitState.GetDeltaTime();

		if (!InstanceData->Input.IsValid())
		{
			InstanceData->Input = Context.AllocateNodeInstance(Binding, SharedData->Input);
		}

		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FOffsetRootBoneTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

#if ENABLE_ANIM_DEBUG
		if(CVarAnimNodeOffsetRootBoneEnable.GetValueOnAnyThread() == 0)
		{
			InstanceData->bIsFirstUpdate = true;
			return;
		}
#endif

		// Need to copy over latent values from shared data to instance data in order to sample all the pin values (via the getters)
		// Todo: wish we didn't have to do this
		InstanceData->Alpha = SharedData->GetAlpha(Binding);
		InstanceData->MeshComponentTransformWorld = SharedData->GetMeshComponentTransformWorld(Binding);
		InstanceData->TranslationMode = SharedData->GetTranslationMode(Binding);
		InstanceData->RotationMode = SharedData->GetRotationMode(Binding);
		InstanceData->MaxTranslationError = SharedData->GetMaxTranslationError(Binding);
		InstanceData->MaxRotationErrorDegrees = SharedData->GetMaxRotationErrorDegrees(Binding);
		InstanceData->TranslationSmoothingTime = SharedData->GetTranslationSmoothingTime(Binding);
		InstanceData->RotationSmoothingTime = SharedData->GetRotationSmoothingTime(Binding);
		InstanceData->bOnGround = SharedData->GetbOnGround(Binding);
		InstanceData->AnimatedGroundNormal = SharedData->GetAnimatedGroundNormal(Binding);

#if ENABLE_ANIM_DEBUG 
		InstanceData->HostObject = Context.GetHostObject();
#endif // ENABLE_ANIM_DEBUG 

		Context.AppendTask(FAnimNextOffsetRootBoneTask::Make(InstanceData, SharedData));
	}

// --- IHierarchy impl --- 
uint32 FOffsetRootBoneTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 1;
	}

void FOffsetRootBoneTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		Children.Add(InstanceData->Input);
	}
} // namespace UE::UAF


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAnimNextOffsetRootBoneTask

FAnimNextOffsetRootBoneTask FAnimNextOffsetRootBoneTask::Make(UE::UAF::FOffsetRootBoneTrait::FInstanceData* InstanceData, const UE::UAF::FOffsetRootBoneTrait::FSharedData* SharedData)
{
	FAnimNextOffsetRootBoneTask Task;
	Task.InstanceData = InstanceData;
	Task.SharedData = SharedData;
	return Task;
}

void FAnimNextOffsetRootBoneTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	QUICK_SCOPE_CYCLE_COUNTER(FAnimNextOffsetRootBoneTask_Execute);
	
	if (InstanceData->DeltaTime == 0.f)
	{
		return;
	}

	if (InstanceData->Alpha == 0.f)
	{
		return;
	}

	// Extract the root motion for this frame
	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (Keyframe == nullptr)
	{
		return;
	}
	
	FTransform RootMotionTransformDelta = FTransform::Identity;
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
	if (!RootMotionProvider)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FAnimNextOffsetRootBoneTask::Execute, missing RootMotionProvider");
		return;
	}

	const FTransform InputBoneTransform = Keyframe->Get()->Pose.LocalTransforms[UE::UAF::FReferencePose::RootBoneIndex];

	const FTransform ComponentTransform = InstanceData->MeshComponentTransformWorld;

	if (InstanceData->bIsFirstUpdate)
	{
		InstanceData->SimulatedTranslation = ComponentTransform.GetTranslation();
		InstanceData->SimulatedRotation = ComponentTransform.GetRotation();
	}

	// Note we set last to current on first update
	FTransform LastComponentTransform = InstanceData->bIsFirstUpdate ? ComponentTransform : InstanceData->LastMeshComponentTransformWorld;
	InstanceData->bIsFirstUpdate = false;
	

	// If the difference between the previous and current component transform is above the teleport threshold
	// then we adjust the simulated position to account for the teleport. Then we set the last transform to be 
	// the same as the previous so that the node does not attempt to counter-animate out the component transform 
	// this frame
	if (FVector::Distance(LastComponentTransform.GetLocation(), ComponentTransform.GetLocation()) > SharedData->TeleportDistanceThreshold)
	{
		if (SharedData->bResetOnTeleport)
		{
			InstanceData->SimulatedTranslation = ComponentTransform.GetTranslation();
			InstanceData->SimulatedRotation = ComponentTransform.GetRotation();
		}
		else
		{
			const FTransform OffsetTransform = FTransform(InstanceData->SimulatedRotation, InstanceData->SimulatedTranslation).GetRelativeTransformReverse(LastComponentTransform);
			const FTransform TeleportedTransform = OffsetTransform * ComponentTransform;
			InstanceData->SimulatedTranslation = TeleportedTransform.GetLocation();
			InstanceData->SimulatedRotation = TeleportedTransform.GetRotation();
		}

		LastComponentTransform = ComponentTransform;
	}

	// Cycle last transform
	InstanceData->LastMeshComponentTransformWorld = InstanceData->MeshComponentTransformWorld;

	const EUAFOffsetRootBoneMode CurrentTranslationMode = InstanceData->TranslationMode;
	const EUAFOffsetRootBoneMode CurrentRotationMode = InstanceData->RotationMode;;

	bool bShouldConsumeTranslationOffset = UE::UAF::OffsetRootBone::ShouldExtractRootMotion(CurrentTranslationMode);
	bool bShouldConsumeRotationOffset = UE::UAF::OffsetRootBone::ShouldExtractRootMotion(CurrentRotationMode);

	if (RootMotionProvider->ExtractRootMotion(Keyframe->Get()->Attributes, RootMotionTransformDelta) == false)
	{
		return;
	}

	RootMotionTransformDelta.NormalizeRotation();
	
	FTransform ConsumedRootMotionDelta = FTransform::Identity;

	if (bShouldConsumeTranslationOffset)
	{
		// Grab root motion translation from the root motion attribute
		ConsumedRootMotionDelta.SetTranslation(RootMotionTransformDelta.GetTranslation());
	}
	if (bShouldConsumeRotationOffset)
	{
		// Grab root motion rotation from the root motion attribute
		ConsumedRootMotionDelta.SetRotation(RootMotionTransformDelta.GetRotation());
	}

	if (UE::UAF::OffsetRootBone::ShouldCounterComponentDelta(CurrentRotationMode))
	{
		// Accumulate the rotation component delta into the simulated rotation, to keep component and offset in sync.
		const FQuat ComponentRotationDelta = LastComponentTransform.GetRotation().Inverse() * ComponentTransform.GetRotation();
		InstanceData->SimulatedRotation = ComponentRotationDelta * InstanceData->SimulatedRotation;
	}
	if (UE::UAF::OffsetRootBone::ShouldCounterComponentDelta(CurrentTranslationMode))
	{
		// Accumulate the translation component delta into the simulated translation, to keep component and offset in sync.
		const FVector ComponentTranslationDelta = ComponentTransform.GetLocation() - LastComponentTransform.GetLocation();
		InstanceData->SimulatedTranslation += ComponentTranslationDelta;
	}

	FTransform SimulatedTransform(InstanceData->SimulatedRotation, InstanceData->SimulatedTranslation);
	// Apply the root motion delta
	SimulatedTransform = ConsumedRootMotionDelta * SimulatedTransform;

	InstanceData->SimulatedTranslation = SimulatedTransform.GetLocation();
	InstanceData->SimulatedRotation = SimulatedTransform.GetRotation();

	if (InstanceData->bOnGround)
	{
		InstanceData->SimulatedTranslation = FVector::PointPlaneProject(InstanceData->SimulatedTranslation, ComponentTransform.GetLocation(), InstanceData->AnimatedGroundNormal);
	}

	if (InstanceData->TranslationMode == EUAFOffsetRootBoneMode::Release ||
		InstanceData->TranslationMode == EUAFOffsetRootBoneMode::Interpolate)
	{
		FVector TranslationOffset = ComponentTransform.GetLocation() - InstanceData->SimulatedTranslation;

		// Blend out translation offset
		FVector TranslationOffsetDelta = FVector::ZeroVector;
		FMath::ExponentialSmoothingApprox(TranslationOffsetDelta,TranslationOffset, InstanceData->DeltaTime, InstanceData->TranslationSmoothingTime);

		if (SharedData->bClampToTranslationVelocity)
		{
			const float RootMotionDelta = RootMotionTransformDelta.GetLocation().Size();
			const float MaxDelta = 	SharedData->TranslationSpeedRatio * RootMotionDelta;

			const float AdjustmentDelta = TranslationOffsetDelta.Size();
			if (AdjustmentDelta > MaxDelta)
			{
				TranslationOffsetDelta = MaxDelta * TranslationOffsetDelta.GetSafeNormal2D();
			}
		}
		
		InstanceData->SimulatedTranslation = InstanceData->SimulatedTranslation + TranslationOffsetDelta;
	}

	if (InstanceData->RotationMode == EUAFOffsetRootBoneMode::Release ||
		InstanceData->RotationMode == EUAFOffsetRootBoneMode::Interpolate)
	{
		FQuat RotationOffset = ComponentTransform.GetRotation() * InstanceData->SimulatedRotation.Inverse();
		RotationOffset.EnforceShortestArcWith(FQuat::Identity);
		FQuat RotationOffsetDelta = FQuat::Identity;
		SpringMath::ExponentialSmoothingApproxQuat( RotationOffsetDelta, RotationOffset, InstanceData->DeltaTime, InstanceData->RotationSmoothingTime);

		if (SharedData->bClampToRotationVelocity)
		{
			float RotationMotionAngleDelta;
			FVector RootMotionRotationAxis;
			RootMotionTransformDelta.GetRotation().ToAxisAndAngle(RootMotionRotationAxis, RotationMotionAngleDelta);

			float MaxRotationAngle = SharedData->RotationSpeedRatio * RotationMotionAngleDelta;

			FVector DeltaAxis;
			float DeltaAngle;
			RotationOffsetDelta.ToAxisAndAngle(DeltaAxis, DeltaAngle);

			if (DeltaAngle > MaxRotationAngle)
			{
				RotationOffsetDelta = FQuat(DeltaAxis, MaxRotationAngle);
			}
		}

		InstanceData->SimulatedRotation = RotationOffsetDelta * InstanceData->SimulatedRotation;
	}

	if (InstanceData->MaxTranslationError >= 0.0f)
	{
		FVector TranslationOffset = ComponentTransform.GetLocation() - InstanceData->SimulatedTranslation;
		const float TranslationOffsetSizeSquared = TranslationOffset.SizeSquared();
		if (TranslationOffsetSizeSquared > (InstanceData->MaxTranslationError * InstanceData->MaxTranslationError))
		{
			TranslationOffset = TranslationOffset.GetClampedToMaxSize(InstanceData->MaxTranslationError);
			InstanceData->SimulatedTranslation = ComponentTransform.GetLocation() - TranslationOffset;
		}
	}

	const float MaxAngleRadians = FMath::DegreesToRadians(InstanceData->MaxRotationErrorDegrees);
	if (MaxAngleRadians >= 0.0f)
	{
		FQuat RotationOffset = ComponentTransform.GetRotation().Inverse() * InstanceData->SimulatedRotation;
		RotationOffset.EnforceShortestArcWith(FQuat::Identity);

		FVector OffsetAxis;
		float OffsetAngle;
		RotationOffset.ToAxisAndAngle(OffsetAxis, OffsetAngle);

		if (FMath::Abs(OffsetAngle) > MaxAngleRadians)
		{
			RotationOffset = FQuat(OffsetAxis, MaxAngleRadians);
			InstanceData->SimulatedRotation = RotationOffset * ComponentTransform.GetRotation();
			InstanceData->SimulatedRotation.Normalize();
		}
	}

	// Apply the offset adjustments to the simulated transform
	SimulatedTransform.SetLocation(InstanceData->SimulatedTranslation);
	SimulatedTransform.SetRotation(InstanceData->SimulatedRotation);

	// Start with the input pose's bone transform, to preserve any adjustments done before this node in the graph
	FTransform TargetBoneTransform = InputBoneTransform;
	// Accumulate the simulated transform in, and counter current component transform.
	TargetBoneTransform.Accumulate(SimulatedTransform * ComponentTransform.Inverse());

	// Offset root bone should not affect scale so take the input
	TargetBoneTransform.SetScale3D(InputBoneTransform.GetScale3D());

	TargetBoneTransform.NormalizeRotation();
	check(InstanceData->SimulatedTranslation.ContainsNaN() == false);

	Keyframe->Get()->Pose.LocalTransforms[UE::UAF::FReferencePose::RootBoneIndex] = TargetBoneTransform;

#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
	if (FVisualLogger::IsRecording())
	{
		static const TCHAR* LogName = TEXT("OffsetRootBone");
		const float InnerCircleRadius = 40.0f;
		const uint16 CircleThickness = 2;
		const FVector CircleOffset(0,0,1);

		const FTransform TargetBoneInitialTransformWorld = InputBoneTransform * ComponentTransform;
		const FTransform TargetBoneTransformWorld = TargetBoneTransform * ComponentTransform;
		
		const UObject* LogOwner = InstanceData->HostObject.Get();

		if (InstanceData->MaxTranslationError >= 0.0f)
		{
			const float OuterCircleRadius = InstanceData->MaxTranslationError + InnerCircleRadius;
			UE_VLOG_CIRCLE_THICK(LogOwner, TEXT("OffsetRootBone"), Display, ComponentTransform.GetLocation() + CircleOffset, FVector::UpVector, OuterCircleRadius, FColor::Red, CircleThickness, TEXT(""));
		}
		
		UE_VLOG_CIRCLE_THICK(LogOwner, LogName, Display, ComponentTransform.GetLocation() + CircleOffset, FVector::UpVector, InnerCircleRadius, FColor::Blue, CircleThickness, TEXT(""));
		UE_VLOG_ARROW(LogOwner, LogName, Display,
			ComponentTransform.GetLocation() + CircleOffset,
			ComponentTransform.GetLocation() + InnerCircleRadius * ComponentTransform.GetRotation().GetRightVector() + CircleOffset,
			FColor::Blue, TEXT(""));
		
		UE_VLOG_CIRCLE_THICK(LogOwner, LogName, Display, TargetBoneTransformWorld.GetLocation() + CircleOffset, FVector::UpVector, InnerCircleRadius, FColor::Green, CircleThickness, TEXT(""));
		UE_VLOG_ARROW(LogOwner, LogName, Display,
		 	TargetBoneTransformWorld.GetLocation() + CircleOffset,
		 	TargetBoneTransformWorld.GetLocation() + InnerCircleRadius * TargetBoneTransformWorld.GetRotation().GetRightVector() + CircleOffset,
			FColor::Green, TEXT(""));
	}
#endif
}


