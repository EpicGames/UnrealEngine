// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"
#include "OffsetRootBoneTrait.generated.h"

UENUM(BlueprintType)
enum class EUAFOffsetRootBoneMode : uint8
{
	// Accumulate the mesh component's movement into the offset.
	// In this mode, if the mesh component moves 
	// the offset will counter the motion, and the root will stay in place
	Accumulate,
	// Continuously interpolate the offset out
	// In this mode, if the mesh component moves
	// The root will stay behind, but will attempt to catch up
	Interpolate,
	// Release the offset and stop accumulating the mesh component's movement delta.
	// In this mode we will "blend out" the offset
	Release,
};


USTRUCT(meta = (DisplayName = "Offset Root Bone"))
struct FOffsetRootBoneTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Input pose to be processed
	UPROPERTY()
	FAnimNextTraitHandle Input;

	// Current strength of the skeletal control
	UPROPERTY(EditAnywhere, Category = Alpha, meta = (PinShownByDefault))
	float Alpha = 1.0f;

	// The translation offset behavior mode
	UPROPERTY(EditAnywhere, Category = Settings)
	EUAFOffsetRootBoneMode TranslationMode = EUAFOffsetRootBoneMode::Interpolate;

	// The rotation offset behavior mode
	UPROPERTY(EditAnywhere, Category = Settings)
	EUAFOffsetRootBoneMode RotationMode = EUAFOffsetRootBoneMode::Interpolate;

	// Controls how fast the translation offset is blended out
	// Values closer to 0 make it faster
	UPROPERTY(EditAnywhere, Category = Settings)
	float TranslationSmoothingTime = 0.1f;

	// Controls how fast the rotation offset is blended out
	// Values closer to 0 make it faster
	UPROPERTY(EditAnywhere, Category = Settings)
	float RotationSmoothingTime = 0.2f;

	// How much the offset can deviate from the mesh component's translation in units
	// Values lower than 0 disable this limit
	UPROPERTY(EditAnywhere, Category = Settings)
	float MaxTranslationError = -1.0f;

	// How much the offset can deviate from the mesh component's rotation in degrees
	// Values lower than 0 disable this limit
	UPROPERTY(EditAnywhere, Category = Settings)
	float MaxRotationErrorDegrees = -1.0f;

	// Whether to limit the offset's translation interpolation speed to the velocity on the incoming motion
	// Enabling this prevents the offset sliding when there's little to no translation speed
	UPROPERTY(EditAnywhere, Category = Settings, meta=(Inline))
	bool bClampToTranslationVelocity = false;

	// Whether to limit the offset's rotation interpolation speed to the velocity on the incoming motion
	// Enabling this prevents the offset sliding when there's little to no rotation speed
	UPROPERTY(EditAnywhere, Category = Settings, meta=(Inline))
	bool bClampToRotationVelocity = false;

	// How much the offset can blend out, relative to the incoming translation speed
	// i.e. If root motion is moving at 400cm/s, at 0.5, the offset can blend out at 200cm/s
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = bClampToTranslationVelocity, Inline))
	float TranslationSpeedRatio = 0.5f;

	// How much the offset can blend out, relative to the incoming rotation speed
	// i.e. If root motion is rotating at 90 degrees/s, at 0.5, the offset can blend out at 45 degree/s
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = bClampToRotationVelocity, Inline))
	float RotationSpeedRatio = 0.5f;

	// @TODO Temp / try to remove this. Shouldn't have to feed as argument
	// This has to be the mesh component transform in world space
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (PinShownByDefault))
	FTransform MeshComponentTransformWorld = FTransform::Identity;

	// Signals whether the offset root bone translation should be projected onto a projected ground plane
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bOnGround = true;

	// Surface normal that the current animation was authored for. Used with the bOnGround flag. 
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector AnimatedGroundNormal = { 0, 0, 1 };

	// Distance threshold used to detect teleports
	UPROPERTY(EditAnywhere, Category = Settings, meta = (Inline))
	float TeleportDistanceThreshold = 300.0f;

	// If to reset the root bone offset on teleport
	UPROPERTY(EditAnywhere, Category = Settings, meta = (Inline))
	bool bResetOnTeleport = false;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Alpha) \
		GeneratorMacro(MeshComponentTransformWorld) \
		GeneratorMacro(TranslationMode) \
		GeneratorMacro(RotationMode) \
		GeneratorMacro(TranslationSmoothingTime) \
		GeneratorMacro(RotationSmoothingTime) \
		GeneratorMacro(MaxTranslationError) \
		GeneratorMacro(MaxRotationErrorDegrees) \
		GeneratorMacro(bOnGround) \
		GeneratorMacro(AnimatedGroundNormal) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FOffsetRootBoneTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

/**
 * This trait adds a procedural offset to the root bone coming from the incoming root motion.
 * It can optionally fade this out over time (controlled by the EUAFOffsetRootBoneMode).
 *
 * This allows for a code controlled character to still be influenced by the root motion of the animation,
 * e.g. during plant turns. It can lead to less foot sliding in a motion matching setup.
 */
namespace UE::UAF
{

struct FOffsetRootBoneTrait : FBaseTrait, IUpdate, IEvaluate, IHierarchy
{
	DECLARE_ANIM_TRAIT(FOffsetRootBoneTrait, FBaseTrait)

	using FSharedData = FOffsetRootBoneTraitSharedData;

	struct FInstanceData : FTrait::FInstanceData
	{
		FTraitPtr Input;

		/** Delta in seconds between updates, populated during PreUpdate */
		float DeltaTime = 0.f;

		float Alpha = 1.0f;

		bool bIsFirstUpdate = true;

		FTransform LastMeshComponentTransformWorld = FTransform::Identity;
		FTransform MeshComponentTransformWorld = FTransform::Identity;

		EUAFOffsetRootBoneMode TranslationMode = EUAFOffsetRootBoneMode::Interpolate;
		EUAFOffsetRootBoneMode RotationMode = EUAFOffsetRootBoneMode::Interpolate;

		float TranslationSmoothingTime = 0.0f;
		float RotationSmoothingTime = 0.0f;
		float MaxTranslationError = -1.0f;
		float MaxRotationErrorDegrees = -1.0f;

		// The simulated world-space transforms for the root bone with offset
		// Offset = ComponentTransform - SimulatedTransform
		FVector SimulatedTranslation = FVector::ZeroVector;
		FQuat SimulatedRotation = FQuat::Identity;

		FVector LastNonZeroRootMotionDirection = FVector::ZeroVector;

		bool bOnGround = true;
		FVector AnimatedGroundNormal = FVector::ZeroVector;
		
#if ENABLE_ANIM_DEBUG 
		/** Debug Object for VisualLogger */
		TObjectPtr<const UObject> HostObject = nullptr;
#endif // ENABLE_ANIM_DEBUG 
	};

	// IUpdate impl 
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

	// IHierarchy impl
	virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
	virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;
};

} // namespace UE::UAF

/** Task to run Offset Root Bone on VM */
USTRUCT()
struct FAnimNextOffsetRootBoneTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextOffsetRootBoneTask)

	static FAnimNextOffsetRootBoneTask Make(UE::UAF::FOffsetRootBoneTrait::FInstanceData* InstanceData, const UE::UAF::FOffsetRootBoneTrait::FSharedData* SharedData);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

private:

	UE::UAF::FOffsetRootBoneTrait::FInstanceData* InstanceData = nullptr;
	const UE::UAF::FOffsetRootBoneTrait::FSharedData* SharedData = nullptr;
};