// Copyright Epic Games, Inc. All Rights Reserved.

#include "OverrideRootMotionTrait.h"

#include "AnimNextWarpingLog.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Math/SpringMath.h"
#include "EvaluationVM/EvaluationVM.h"
#include "VisualLogger/VisualLogger.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FOverrideRootMotionTrait

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FOverrideRootMotionTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \

	// Trait implementation boilerplate
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FOverrideRootMotionTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FOverrideRootMotionTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FOverrideRootMotionTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		Context.AppendTask(FAnimNextOverrideRootMotionTask::Make(SharedData->GetAlpha(Binding), SharedData->GetOverrideRootMotionDelta(Binding), SharedData->GetOverrideRootMotionMode(Binding)));
	}

} // namespace UE::UAF


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAnimNextOverrideRootMotionTask

FAnimNextOverrideRootMotionTask FAnimNextOverrideRootMotionTask::Make(float Alpha, FTransform OverrideRootMotionDelta, EUAFOverrideRootMotionMode OverrideRootMotionMode)
{
	FAnimNextOverrideRootMotionTask Task;
	Task.Alpha = Alpha;
	Task.OverrideRootMotionDelta = OverrideRootMotionDelta;
	Task.OverrideRootMotionMode = OverrideRootMotionMode;
	return Task;
}

void FAnimNextOverrideRootMotionTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	QUICK_SCOPE_CYCLE_COUNTER(FAnimNextOffsetRootBoneTask_Execute);

	if (Alpha < UE_SMALL_NUMBER)
	{
		return;
	}

	// Extract the root motion for this frame
	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (Keyframe == nullptr || Keyframe->Get() == nullptr)
	{
		return;
	}
	
	FTransform RootMotionTransformDelta = FTransform::Identity;
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
	if (!RootMotionProvider)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FAnimNextOverrideRootMotionTask::Execute, missing RootMotionProvider");
		return;
	}

	bool RequiresBlend = Alpha < 1.0f - UE_SMALL_NUMBER;
	// We only need to check the incoming root motion if we are going to blend it
	// Or add it to the override
	if (RequiresBlend || OverrideRootMotionMode == EUAFOverrideRootMotionMode::Additive)
	{
		// Note: this can fail, but if so we just use identity as the incoming root motion delta
		RootMotionProvider->ExtractRootMotion(Keyframe->Get()->Attributes, RootMotionTransformDelta);
	}

	FTransform NewRootMotionToApply;

	if (OverrideRootMotionMode == EUAFOverrideRootMotionMode::Replace)
	{
		NewRootMotionToApply = OverrideRootMotionDelta;
	}
	else
	{
		NewRootMotionToApply = RootMotionTransformDelta; 
		NewRootMotionToApply.NormalizeRotation();
		NewRootMotionToApply.Accumulate(OverrideRootMotionDelta);
	}

	if (RequiresBlend)
	{
		// Need to blend effect on
		NewRootMotionToApply.Blend(RootMotionTransformDelta, NewRootMotionToApply, Alpha);
	}

	RootMotionProvider->SetRootMotion(NewRootMotionToApply, Keyframe->Get()->Attributes);
	
}


