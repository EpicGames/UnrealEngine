// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "OverrideRootMotionTrait.generated.h"

UENUM(BlueprintType)
enum class EUAFOverrideRootMotionMode : uint8
{
	Replace,
	Additive
};


USTRUCT(meta = (DisplayName = "Override Root Motion"))
struct FOverrideRootMotionTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Current strength of the override
	UPROPERTY(EditAnywhere, Category = Settings)
	float Alpha = 1.0f;

	UPROPERTY(EditAnywhere, Category = Settings)
	FTransform OverrideRootMotionDelta = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = Settings)
	EUAFOverrideRootMotionMode OverrideRootMotionMode = EUAFOverrideRootMotionMode::Replace;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Alpha) \
		GeneratorMacro(OverrideRootMotionDelta) \
		GeneratorMacro(OverrideRootMotionMode) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FOverrideRootMotionTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{

/**
 * Override the root motion attributes with custom values.
 * 
 * Ex: allows code driven motion to be injected into the animation system as root motion.
 */
struct FOverrideRootMotionTrait : FAdditiveTrait, IUpdate, IEvaluate
{
	DECLARE_ANIM_TRAIT(FOverrideRootMotionTrait, FAdditiveTrait)

	using FSharedData = FOverrideRootMotionTraitSharedData;

	struct FInstanceData : FTrait::FInstanceData
	{
	};

	// IUpdate impl 
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
};

} // namespace UE::UAF

/** Task to run override root motion on VM */
USTRUCT()
struct FAnimNextOverrideRootMotionTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextOverrideRootMotionTask)

	static FAnimNextOverrideRootMotionTask Make(float Alpha, FTransform OverrideRootMotionDelta, EUAFOverrideRootMotionMode OverrideRootMotionMode);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

private:

	float Alpha = 1.0f;
	FTransform OverrideRootMotionDelta = FTransform::Identity;
	EUAFOverrideRootMotionMode OverrideRootMotionMode = EUAFOverrideRootMotionMode::Replace;
};