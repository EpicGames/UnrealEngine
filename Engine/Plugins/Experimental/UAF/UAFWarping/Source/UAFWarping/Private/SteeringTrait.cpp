// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteeringTrait.h"

#include "AnimNextWarpingLog.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequenceBase.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IGraphFactory.h"
#include "TraitInterfaces/ITimeline.h"
#include "VisualLogger/VisualLogger.h"
#include "Math/SpringMath.h"
#include "TraitInterfaces/IHierarchy.h"

bool bAnimNextSteeringTraitEnabled = true;
static FAutoConsoleVariableRef CVarAnimNextSteeringTraitEnabled(
	TEXT("a.AnimNext.SteeringTrait.Enabled"),
	bAnimNextSteeringTraitEnabled,
	TEXT("True will enable steering for AnimNext. Equivalent to setting alpha to non-zero.")
);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FSteeringTrait

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FSteeringTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \

	// Trait implementation boilerplate
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FSteeringTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FSteeringTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		InstanceData->DeltaTime = TraitState.GetDeltaTime();

		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FSteeringTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		// Get Current Anim Asset
		{
			TTraitBinding<IAttributeProvider> AttributeTrait;
			FTraitStackBinding StackBinding;
			if (IHierarchy::GetForwardedStackInterface<IAttributeProvider>(Context, Binding, StackBinding, AttributeTrait))
			{
  				InstanceData->OnExtractRootMotionAttribute = AttributeTrait.GetOnExtractRootMotionAttribute(Context);
			}
		}

		// Get Current Anim Asset Time
		{
			TTraitBinding<ITimeline> TimelineTrait;
			FTimelineState TimelineState;
			FTraitStackBinding StackBinding;
			if (IHierarchy::GetForwardedStackInterface<ITimeline>(Context, Binding, StackBinding, TimelineTrait) && TimelineTrait.GetState(Context, TimelineState))
			{
				InstanceData->CurrentAnimAssetTime = TimelineState.GetPosition();
			}
			else
			{
				// Set time to be -1.0 to indicate we should skip root motion prediction consideration
				InstanceData->CurrentAnimAssetTime = -1.0f;
			}
		}

		// Update target orientation, root bone transform, & other properties 
		InstanceData->TargetOrientation = SharedData->GetTargetOrientation(Binding);
		InstanceData->RootBoneTransform = SharedData->GetRootBoneTransform(Binding);
		InstanceData->Alpha = SharedData->GetAlpha(Binding);

#if ENABLE_ANIM_DEBUG 
		InstanceData->HostObject = Context.GetHostObject();
#endif // ENABLE_ANIM_DEBUG 

		Context.AppendTask(FAnimNextSteeringTask::Make(InstanceData, SharedData));
	}
} // namespace UE::UAF


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAnimNextSteeringTask

FAnimNextSteeringTask FAnimNextSteeringTask::Make(UE::UAF::FSteeringTrait::FInstanceData* InstanceData, const UE::UAF::FSteeringTrait::FSharedData* SharedData)
{
	FAnimNextSteeringTask Task;
	Task.InstanceData = InstanceData;
	Task.SharedData = SharedData;
	return Task;
}

void FAnimNextSteeringTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	// Execute Steering
	if (InstanceData->DeltaTime > 0.f)
	{
		if (InstanceData->Alpha > 0.0f && bAnimNextSteeringTraitEnabled)
		{
			const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
			if (!RootMotionProvider)
			{
				UE_LOGF(LogAnimNextWarping, Error, "FAnimNextSteeringTask::Execute, missing RootMotionProvider");
				return;
			}

			if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
			{
				FTransform ThisFrameRootMotionTransform = FTransform::Identity;
				if (RootMotionProvider->ExtractRootMotion(Keyframe->Get()->Attributes, ThisFrameRootMotionTransform))
				{
					float CurrentSpeed = ThisFrameRootMotionTransform.GetTranslation().Length() / InstanceData->DeltaTime;
					if (CurrentSpeed > SharedData->DisableSteeringBelowSpeed)
					{
						FQuat RootBoneRotation = InstanceData->RootBoneTransform.GetRotation();

#if ENABLE_ANIM_DEBUG
						UE_VLOG_ARROW(InstanceData->HostObject, "Steering", Display,
							InstanceData->RootBoneTransform.GetLocation(),
							InstanceData->RootBoneTransform.GetLocation() + RootBoneRotation.GetRightVector() * 90,
							FColor::Green, TEXT(""));

						UE_VLOG_ARROW(InstanceData->HostObject, "Steering", Display,
							InstanceData->RootBoneTransform.GetLocation(),
							InstanceData->RootBoneTransform.GetLocation() + InstanceData->TargetOrientation.GetRightVector() * 100,
							FColor::Blue, TEXT(""));
#endif // ENABLE_ANIM_DEBUG 

						FQuat DeltaToTargetOrientation = RootBoneRotation.Inverse() * InstanceData->TargetOrientation;

						if (SharedData->AnimatedTargetTime > 0.0f)
						{
							if (InstanceData->OnExtractRootMotionAttribute.IsBound() && InstanceData->CurrentAnimAssetTime != -1.0f)
							{
								check(InstanceData->CurrentAnimAssetTime >= 0.0f);
								FTransform PredictedRootMotionTransform = InstanceData->OnExtractRootMotionAttribute.Execute(InstanceData->CurrentAnimAssetTime, SharedData->AnimatedTargetTime, true);
								FQuat PredictedRootMotionQuat = PredictedRootMotionTransform.GetRotation();
								FRotator PredictedRootMotionRot = FRotator(PredictedRootMotionQuat);
								float PredictedRootMotionYaw = PredictedRootMotionRot.Yaw;

								if (fabs(PredictedRootMotionYaw) > SharedData->RootMotionThreshold)
								{
#if ENABLE_ANIM_DEBUG
									UE_VLOG_ARROW(InstanceData->HostObject, "Steering", Display,
										InstanceData->RootBoneTransform.GetLocation(),
										InstanceData->RootBoneTransform.GetLocation() + (PredictedRootMotionQuat * RootBoneRotation).GetRightVector() * 100,
										FColor::Orange, TEXT(""));
#endif // ENABLE_ANIM_DEBUG 

									float YawToTargetOrientation = FRotator(DeltaToTargetOrientation).Yaw;

									// pick the rotation direction that is the shortest path from the endpoint of the current animated rotation
									if (PredictedRootMotionYaw - YawToTargetOrientation > 180)
									{
										YawToTargetOrientation += 360;
									}
									else if (YawToTargetOrientation - PredictedRootMotionYaw > 180)
									{
										YawToTargetOrientation -= 360;
									}

									float Ratio = YawToTargetOrientation / PredictedRootMotionYaw;
									Ratio = FMath::Clamp(Ratio, SharedData->MinScaleRatio, SharedData->MaxScaleRatio);

									// Account for alpha
									Ratio = FMath::Lerp(1.0f, Ratio, InstanceData->Alpha);

									FRotator ThisFrameRootMotionRotator(ThisFrameRootMotionTransform.GetRotation());
									ThisFrameRootMotionRotator.Yaw *= Ratio;
									ThisFrameRootMotionTransform.SetRotation(FQuat(ThisFrameRootMotionRotator));

									// Account for future scaling in linear error correction
									PredictedRootMotionRot.Yaw *= Ratio;
									PredictedRootMotionQuat = PredictedRootMotionRot.Quaternion();

									DeltaToTargetOrientation = PredictedRootMotionQuat.Inverse() * RootBoneRotation.Inverse() * InstanceData->TargetOrientation;
								}
							}
						}

						if (CurrentSpeed > SharedData->DisableAdditiveBelowSpeed)
						{
							// Apply linear correction
							FQuat LinearCorrection = FQuat::Identity;
							SpringMath::CriticalSpringDamperQuat(LinearCorrection, InstanceData->AngularVelocity, DeltaToTargetOrientation, SpringMath::HalfLifeToSmoothingTime(SharedData->ProceduralTargetTime), InstanceData->DeltaTime);

#if ENABLE_ANIM_DEBUG
							UE_VLOG_ARROW(InstanceData->HostObject, "Steering", Display,
								InstanceData->RootBoneTransform.GetLocation(),
								InstanceData->RootBoneTransform.GetLocation() + (InstanceData->RootBoneTransform.GetRotation() * LinearCorrection).GetRightVector() * 120,
								FColor::Magenta, TEXT(""));
#endif

							FQuat ThisFrameRotation = ThisFrameRootMotionTransform.GetRotation() * LinearCorrection;

#if ENABLE_ANIM_DEBUG
							UE_VLOG_ARROW(InstanceData->HostObject, "Steering", Display,
								InstanceData->RootBoneTransform.GetLocation(),
								InstanceData->RootBoneTransform.GetLocation() + (InstanceData->RootBoneTransform.GetRotation() * ThisFrameRotation).GetRightVector() * 140,
								FColor::Red, TEXT(""));
#endif

							ThisFrameRootMotionTransform.SetRotation(FQuat::Slerp(ThisFrameRootMotionTransform.GetRotation(), ThisFrameRotation, InstanceData->Alpha));
						}

						RootMotionProvider->OverrideRootMotion(ThisFrameRootMotionTransform, Keyframe->Get()->Attributes);
					}
				}
			}
		}
	}
}
