// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Kismet/KismetMathLibrary.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "SteeringTrait.generated.h"

/**
 * Data needed to execute steering
 * 
 * Some steering data such as current anim asset / playback time is acquired via trait stack interfaces
 * 
 * Add procedural delta to the root motion attribute 
 * This is done via 2 techniques:
 *  1) Scaling the root motion on an animation
 *  2) Adding additional correction to root motion after accounting for the anticipated (potentially scaled) root motion
 * The effects of 1) and 2) combine
 */
USTRUCT(meta = (DisplayName = "Steering"))
struct FSteeringTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// @TODO: This should belong in a parent class possibly? Discuss review, make JIRA
	// Current strength of the skeletal control
	UPROPERTY(EditAnywhere, Category = Alpha, meta = (PinShownByDefault))
	float Alpha = 1.0f;

	// The Orientation to steer towards
	UPROPERTY(EditAnywhere, Category=Evaluation, meta=(PinShownByDefault))
	FQuat TargetOrientation = FQuat::Identity;

	// @TODO Temp / try to remove this. Shouldn't have to feed as argument
	// Last root bone transform sampled
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (PinShownByDefault))
	FTransform RootBoneTransform = FTransform::Identity;

	// The number of seconds in the future before we should reach the TargetOrientation when play animations with no root motion rotation
	UPROPERTY(EditAnywhere, Category=Evaluation, meta=(Inline))
	float ProceduralTargetTime = 0.2f;
	
	// The number of seconds in the future before we should reach the TargetOrientation when playing animations with root motion rotation
	UPROPERTY(EditAnywhere, Category=Evaluation, meta=(Inline))
	float AnimatedTargetTime = 0.2f;

	// The minimum amount of root motion required to enable root motion scaling.
	// The root motion is measured from current time to AnimatedTargetTime
	UPROPERTY(EditAnywhere, DisplayName=RootMotionAngleThreshold, Category=Evaluation, meta=(Inline))
	float RootMotionThreshold = 1.0f;

	// below this movement speed (based on the root motion in the animation) disable steering completely (both scaling and additive)
	UPROPERTY(EditAnywhere, Category=Evaluation, meta=(Inline))
	float DisableSteeringBelowSpeed = 1.0f;

	// below this movement speed (based on the root motion in the animation) disable steering coming from the additive spring based correction
	UPROPERTY(EditAnywhere, Category = Evaluation, Meta=(Inline))
	float DisableAdditiveBelowSpeed = -1.0f;

	// Will clamp the scaling ratio applied to above this threashold. Any error remaining will be compensated for linearly (using ProceduralTargetTime)
	UPROPERTY(EditAnywhere, Category = Evaluation, Meta = (Inline))
	float MinScaleRatio = 0.5f;

	// Will clamp the scaling ratio applied to below this threashold. Any error remaining will be compensated for linearly (using ProceduralTargetTime)
	UPROPERTY(EditAnywhere, Category = Evaluation, Meta = (Inline))
	float MaxScaleRatio = 1.5f;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Alpha) \
		GeneratorMacro(TargetOrientation) \
		GeneratorMacro(RootBoneTransform) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FSteeringTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

/**
 * Add procedural delta to root motion to match target orientations
 * 
 * Ex: If your anim only rotates say 45 deg, but you need to warp it to 60 deg to match gameplay input
 */
namespace UE::UAF
{

struct FSteeringTrait : FAdditiveTrait, IUpdate, IEvaluate
{
	DECLARE_ANIM_TRAIT(FSteeringTrait, FAdditiveTrait)

	using FSharedData = FSteeringTraitSharedData;

	struct FInstanceData : FTrait::FInstanceData
	{
		/** Target orientation per instance */
		FQuat TargetOrientation = FQuat::Identity;

		/** Callback provided by attribute trait on stack to evaluate root motion at a later time */
		FOnExtractRootMotionAttribute OnExtractRootMotionAttribute = FOnExtractRootMotionAttribute();

	
		/** Angular velocity of additive correction spring */
		FVector AngularVelocity = FVector::ZeroVector;

		/** Last root bone transform sampled */
		FTransform RootBoneTransform = FTransform::Identity;

		/** Current Anim Asset time, used with ProceduralTargetTime to get future root motion, populated during PostEvaluate  */
		float CurrentAnimAssetTime = 0.f;

		/** Delta in seconds between updates, populated during PreUpdate */
		float DeltaTime = 0.f;

		float Alpha = 1.0f;

#if ENABLE_ANIM_DEBUG 
		/** Debug Object for VisualLogger */
		TObjectPtr<const UObject> HostObject = nullptr;
#endif // ENABLE_ANIM_DEBUG 
	};

	// IUpdate impl 
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
};

} // namespace UE::UAF

/** Task to run Steering on VM */
USTRUCT()
struct FAnimNextSteeringTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextSteeringTask)

	static FAnimNextSteeringTask Make(UE::UAF::FSteeringTrait::FInstanceData* InstanceData, const UE::UAF::FSteeringTrait::FSharedData* SharedData);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UE::UAF::FSteeringTrait::FInstanceData* InstanceData = nullptr;
	const UE::UAF::FSteeringTrait::FSharedData* SharedData = nullptr;
};