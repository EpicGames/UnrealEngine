// Copyright Epic Games, Inc. All Rights Reserved.

#include "StrafeWarpingTrait.h"

#include "AnimNextWarpingLog.h"
#include "TwoBoneIK.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequenceBase.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IGraphFactory.h"
#include "TraitInterfaces/ITimeline.h"
#include "VisualLogger/VisualLogger.h"

#if ENABLE_ANIM_DEBUG
bool bAnimNextStrafeWarpingTraitEnabled = true;
static FAutoConsoleVariableRef CVarAnimNextStrafeWarpingTraitEnabled(
	TEXT("a.AnimNext.StrafeWarpingTrait.Enabled"),
	bAnimNextStrafeWarpingTraitEnabled,
	TEXT("True will enable strafe warping for AnimNext. Equivalent to setting alpha to non-zero.")
);
#else
constexpr bool bAnimNextStrafeWarpingTraitEnabled = true;
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FStrafeWarpingTrait

namespace UE::UAF
{
	static FVector GetAxisVector(const EAxis::Type& InAxis)
	{
		switch (InAxis)
		{
		case EAxis::X:
			return FVector::ForwardVector;
		case EAxis::Y:
			return FVector::RightVector;
		default:
			return FVector::UpVector;
		};
	}

	static float SignedAngleRadBetweenNormals(const FVector& From, const FVector& To, const FVector& Axis)
	{
		const float FromDotTo = FVector::DotProduct(From, To);
		const float Angle = FMath::Acos(FromDotTo);
		const FVector Cross = FVector::CrossProduct(From, To);
		const float Dot = FVector::DotProduct(Cross, Axis);
		return Dot >= 0 ? Angle : -Angle;
	}
	
	AUTO_REGISTER_ANIM_TRAIT(FStrafeWarpingTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \

	// Trait implementation boilerplate
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FStrafeWarpingTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FStrafeWarpingTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		InstanceData->DeltaTime = TraitState.GetDeltaTime();

		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FStrafeWarpingTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		InstanceData->RootBoneTransform = SharedData->GetRootBoneTransform(Binding);

		// Update target orientation, root bone transform, & other properties 
		InstanceData->TargetOrientation = SharedData->GetTargetOrientation(Binding);
		InstanceData->Alpha = SharedData->GetAlpha(Binding);

#if ENABLE_ANIM_DEBUG 
		InstanceData->HostObject = Context.GetHostObject();
#endif // ENABLE_ANIM_DEBUG 

		Context.AppendTask(FAnimNextStrafeWarpingTask::Make(InstanceData, SharedData));
	}

	void FStrafeWarpingTrait::InitializeSpineData(TArrayView<FSpineBoneData> OutSpineBoneData, const TArray<FName>& SpineBoneNames, const FLODPoseStack& Pose)
	{
		QUICK_SCOPE_CYCLE_COUNTER(FStrafeWarpingTrait_InitializeSpineData);
		check(OutSpineBoneData.Num() == SpineBoneNames.Num())
		
		const FReferencePose& RefPose = *Pose.RefPose;

		if (SpineBoneNames.Num() == 0)
		{
			return;
		}
		
		for (int32 i = 0; i < SpineBoneNames.Num(); i++)
		{
			OutSpineBoneData[i].Weight = 0.0f;
			if (const FBoneIndexType* SpineBoneIndex = RefPose.GetBoneNameToLODBoneIndexMap().Find(SpineBoneNames[i]))
			{
				OutSpineBoneData[i].LODBoneIndex = *SpineBoneIndex;
			}
			else
			{
				OutSpineBoneData[i].LODBoneIndex = INVALID_BONE_INDEX;
			}
		}

		// Calculate weight

		// Sort bones indices so we can transform parent before child
		OutSpineBoneData.Sort(FSpineBoneData::FCompareBoneIndex());

		// Assign Weights.
		TArray<int32, TInlineAllocator<20>> IndicesToUpdate;

		// Note reverse iteration
		for (int32 Index = OutSpineBoneData.Num() - 1; Index >= 0; Index--)
		{
			// If this bone's weight hasn't been updated, scan its parents.
			// If parents have weight, we add it to 'ExistingWeight'.
			// split (1.f - 'ExistingWeight') between all members of the chain that have no weight yet.
			if (OutSpineBoneData[Index].Weight == 0.f)
			{
				IndicesToUpdate.Reset(OutSpineBoneData.Num());
				float ExistingWeight = 0.f;
				IndicesToUpdate.Add(Index);

				for (int32 ParentIndex = Index - 1; ParentIndex >= 0; ParentIndex--)
				{
					if (Pose.IsBoneChildOf(OutSpineBoneData[Index].LODBoneIndex, OutSpineBoneData[ParentIndex].LODBoneIndex))
					{
						if (OutSpineBoneData[ParentIndex].Weight > 0.f)
						{
							ExistingWeight += OutSpineBoneData[ParentIndex].Weight;
						}
						else
						{
							IndicesToUpdate.Add(ParentIndex);
						}
					}
				}

				check(IndicesToUpdate.Num() > 0);
				const float WeightToShare = 1.f - ExistingWeight;
				const float IndividualWeight = WeightToShare / float(IndicesToUpdate.Num());

				for (int32 UpdateListIndex = 0; UpdateListIndex < IndicesToUpdate.Num(); UpdateListIndex++)
				{
					OutSpineBoneData[IndicesToUpdate[UpdateListIndex]].Weight = IndividualWeight;
				}
			}
		}
	}
} // namespace UE::UAF


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAnimNextSteeringTask

FAnimNextStrafeWarpingTask FAnimNextStrafeWarpingTask::Make(UE::UAF::FStrafeWarpingTrait::FInstanceData* InstanceData, const UE::UAF::FStrafeWarpingTrait::FSharedData* SharedData)
{
	FAnimNextStrafeWarpingTask Task;
	Task.InstanceData = InstanceData;
	Task.SharedData = SharedData;
	return Task;
}



void FAnimNextStrafeWarpingTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	QUICK_SCOPE_CYCLE_COUNTER(FAnimNextStrafeWarpingTask_Execute);

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}
	
	if (InstanceData->DeltaTime == 0.f)
	{
		return;
	}

	if (InstanceData->Alpha == 0.f || !bAnimNextStrafeWarpingTraitEnabled)
	{
		return;
	}

	float TargetOrientationAngleRad = 0.0f;
	const FVector RotationAxisVector = GetAxisVector(SharedData->RotationAxis);

	// Target Orientation is in world space, transform to root
	FQuat TargetRotation = InstanceData->RootBoneTransform.GetRotation().Inverse() * InstanceData->TargetOrientation; 
	FVector TargetMoveDir = TargetRotation.GetForwardVector();

	// Flatten locomotion direction, along the rotation axis.
	TargetMoveDir = (TargetMoveDir - RotationAxisVector.Dot(TargetMoveDir) * RotationAxisVector).GetSafeNormal();

	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (Keyframe == nullptr)
	{
		return;
	}

	DoWarpRootMotion(*Keyframe, RotationAxisVector, TargetMoveDir, OUT TargetOrientationAngleRad);

	// Calculate the orientation warp angle for pose adjustments (spine and foot IK)
	const float MaxAngleCorrectionRad = FMath::DegreesToRadians(SharedData->MaxCorrectionDegrees);
	
	// Optionally interpolate the effective orientation towards the target orientation angle
	// When the orientation warping node becomes relevant, the input pose orientation may not be aligned with the desired orientation.
	// Instead of interpolating this difference, snap to the desired orientation if it's our first update to minimize corrections over-time.
	if ((SharedData->RotationInterpSpeed > 0.f) /*&& !bIsFirstUpdate */)
	{
		const float SmoothOrientationAngleRad = FMath::FInterpTo(InstanceData->OrientationAngleForPoseWarpRad, TargetOrientationAngleRad, InstanceData->DeltaTime, SharedData->RotationInterpSpeed);
		// Limit our interpolation rate to prevent pops.
		// @TODO: Use better, more physically accurate interpolation here.
		InstanceData->OrientationAngleForPoseWarpRad = FMath::Clamp(SmoothOrientationAngleRad, InstanceData->OrientationAngleForPoseWarpRad - MaxAngleCorrectionRad, InstanceData->OrientationAngleForPoseWarpRad + MaxAngleCorrectionRad);
	}
	else
	{
		InstanceData->OrientationAngleForPoseWarpRad = TargetOrientationAngleRad;
	}

	InstanceData->OrientationAngleForPoseWarpRad = FMath::Clamp(InstanceData->OrientationAngleForPoseWarpRad, -MaxAngleCorrectionRad, MaxAngleCorrectionRad);
	// Allow the alpha value of the node to affect the final rotation
	InstanceData->OrientationAngleForPoseWarpRad *= InstanceData->Alpha;

	if (FMath::IsNearlyZero(InstanceData->OrientationAngleForPoseWarpRad, KINDA_SMALL_NUMBER))
	{
		// No strafe angle, early out before hitting the pose modification code
		return;
	}

	DoWarpPose(*Keyframe, RotationAxisVector);
}

void FAnimNextStrafeWarpingTask::DoWarpRootMotion(TUniquePtr<UE::UAF::FKeyframeState>& Keyframe, const FVector& RotationAxisVector, const FVector& TargetMoveDir, float& OutTargetOrientationAngleRad) const
{
	FTransform ThisFrameRootMotionTransform = FTransform::Identity;
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
	if (!RootMotionProvider)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FAnimNextStrafeWarpingTask::Execute, missing RootMotionProvider");
		return;
	}

	if (RootMotionProvider->ExtractRootMotion(Keyframe.Get()->Attributes, ThisFrameRootMotionTransform))
	{
		FVector RootMotionDeltaTranslation = ThisFrameRootMotionTransform.GetTranslation();
		const FQuat PreviousRootMotionDeltaRotation = InstanceData->RootMotionDeltaRotation;
		InstanceData->RootMotionDeltaRotation = ThisFrameRootMotionTransform.GetRotation();
		
		const float RootMotionDeltaSpeed = RootMotionDeltaTranslation.Size() / InstanceData->DeltaTime;
		if (RootMotionDeltaSpeed < SharedData->MinRootMotionSpeedThreshold)
		{
			// If we're under the threshold, snap orientation angle to 0, and let interpolation handle the delta
			OutTargetOrientationAngleRad = 0.0f;
		}
		else
		{
			const FVector PreviousRootMotionDeltaDirection = InstanceData->RootMotionDeltaDirection;
			// Hold previous direction if we can't calculate it from current move delta, because the root is no longer moving
			InstanceData->RootMotionDeltaDirection = RootMotionDeltaTranslation.GetSafeNormal(UE_SMALL_NUMBER, PreviousRootMotionDeltaDirection);
			OutTargetOrientationAngleRad = UE::UAF::SignedAngleRadBetweenNormals(
				InstanceData->RootMotionDeltaDirection, TargetMoveDir, RotationAxisVector);

			// Motion Matching may return an animation that deviates a lot from the movement direction (e.g movement direction going bwd and motion matching could return the fwd animation for a few frames)
			// When that happens, since we use the delta between root motion and movement direction, we would be over-rotating the lower body and breaking the pose during those frames
			// So, when that happens we use the inverse of the root motion direction to calculate our target rotation. 
			// This feels a bit 'hacky' but its the only option I've found so far to mitigate the problem
			if (SharedData->LocomotionAngleDeltaThreshold > 0.f)
			{
				if (FMath::Abs(FMath::RadiansToDegrees(OutTargetOrientationAngleRad)) > SharedData->LocomotionAngleDeltaThreshold)
				{
					OutTargetOrientationAngleRad = FMath::UnwindRadians(OutTargetOrientationAngleRad + FMath::DegreesToRadians(180.0f));
					InstanceData->RootMotionDeltaDirection = -InstanceData->RootMotionDeltaDirection;
				}
			}
			/* No prediction in first iteration for AnimNext. This code is copy-pasta from AnimNode_OrientationWarping
			// If there is translation in predicted root motion, use it if the orientation error is less than current error
			if (TargetTime > 0 && CurrentAnimAsset && !PredictedRootMotionDeltaTranslation.IsNearlyZero(UE_SMALL_NUMBER))
			{
				PredictedRootMotionDeltaTranslation.Normalize();
				float PredictedOrientationErrorAngleRad = UE::Anim::SignedAngleRadBetweenNormals(
					PredictedRootMotionDeltaTranslation, LocomotionForward, RotationAxisVector);

				// The future orientation will often match the current, so add a small delta to avoid further testing for same values
				if (FMath::Abs(PredictedOrientationErrorAngleRad) + UE_KINDA_SMALL_NUMBER < FMath::Abs(TargetOrientationAngleRad))
				{
					// Note: Don't update root motion direction, as the root motion direction is what we are playing, 
					// not the future. Updating root motion direction will break counter compensate. 
					// Also, we rely on the built in interp for continuity / smoothness
#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG
					FutureRootMotionDeltaDirection = PredictedRootMotionDeltaTranslation;
					bUsedFutureRootMotion = true;
#endif
					TargetOrientationAngleRad = PredictedOrientationErrorAngleRad;
				}
			}
			*/

			// Don't compensate interpolation by the root motion angle delta if the previous direction isn't valid.
			if (SharedData->bCounterCompenstateInterpolationByRootMotion && !PreviousRootMotionDeltaDirection.IsNearlyZero(UE_SMALL_NUMBER))
			{
				float RootMotionDeltaAngleRad = 0.0f;
				// Counter the interpolated orientation angle by the root motion direction angle delta.
				// This prevents our interpolation from fighting the natural root motion that's flowing through the graph.
				// To correctly measure the amount to counter, we need to unrotate our previous delta direction by our previous rotation
				// As the previous direction delta is relative to the previous rotation delta
				RootMotionDeltaAngleRad = UE::UAF::SignedAngleRadBetweenNormals(
					InstanceData->RootMotionDeltaDirection, PreviousRootMotionDeltaRotation.UnrotateVector(PreviousRootMotionDeltaDirection),
					RotationAxisVector);

				// Root motion may have large deltas i.e. bad blends or sudden direction changes like pivots.
				// If there's an instantaneous pop in root motion direction, this is likely a pivot.
				const float MaxRootMotionDeltaToCompensateRad = FMath::DegreesToRadians(SharedData->MaxRootMotionDeltaToCompensateDegrees);
				if (FMath::Abs(RootMotionDeltaAngleRad) < MaxRootMotionDeltaToCompensateRad)
				{
					InstanceData->CounterCompensateTargetAngleRad += RootMotionDeltaAngleRad;
					float CounterCompensateAngle = FMath::FInterpTo(0, InstanceData->CounterCompensateTargetAngleRad, InstanceData->DeltaTime,
					                                                SharedData->CounterCompensateInterpSpeed);
					InstanceData->OrientationAngleForPoseWarpRad = FMath::UnwindRadians(InstanceData->OrientationAngleForPoseWarpRad + CounterCompensateAngle);
					InstanceData->CounterCompensateTargetAngleRad -= CounterCompensateAngle;
				}
			}

			// Rotate the root motion delta fully by the warped angle
			const FVector WarpedRootMotionTranslationDelta = FQuat(RotationAxisVector, OutTargetOrientationAngleRad).RotateVector(
				RootMotionDeltaTranslation);
			ThisFrameRootMotionTransform.SetTranslation(WarpedRootMotionTranslationDelta);
			
			RootMotionProvider->OverrideRootMotion(ThisFrameRootMotionTransform, Keyframe.Get()->Attributes);
		}
	}

#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
	if (FVisualLogger::IsRecording())
	{
		constexpr float DebugDrawScale = 1.f;
		const FTransform ComponentTransform = InstanceData->RootBoneTransform;
		
		FVector DebugArrowOffset = FVector::ZAxisVector * DebugDrawScale;
		//uint8 DebugAlpha = AnimNodeOrientationWarpingDebugTransparency ? 255 * BlendWeight : 255;
		uint8 DebugAlpha = 255;
		FColor DebugColor = FColor::Green;

		// Draw debug shapes
		{
			const FVector ForwardDirection = ComponentTransform.GetRotation().RotateVector(TargetMoveDir);

			UE_VLOG_CIRCLE_THICK(InstanceData->HostObject, "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset + ForwardDirection * 100.f * DebugDrawScale,
				FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
			UE_VLOG_ARROW(InstanceData->HostObject, "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + ForwardDirection * 100.f * DebugDrawScale,
				FColor::Red.WithAlpha(DebugAlpha), TEXT(""));

			const FVector RotationDirection = ComponentTransform.GetRotation().RotateVector(InstanceData->RootMotionDeltaDirection);

			DebugArrowOffset += FVector::ZAxisVector * DebugDrawScale;

			UE_VLOG_CIRCLE_THICK(InstanceData->HostObject, "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset + RotationDirection * 100.f * DebugDrawScale,
				FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
			UE_VLOG_ARROW(InstanceData->HostObject, "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + RotationDirection * 100.f * DebugDrawScale,
				FColor::Blue.WithAlpha(DebugAlpha), TEXT(""));

			/* Debug for predictive root motion, not currently implemented
			if (bUsedFutureRootMotion && bGraphDrivenWarping)
			{
				const FVector FutureRotationDirection = ComponentTransform.GetRotation().RotateVector(FutureRootMotionDeltaDirection);

				UE_VLOG_CIRCLE_THICK(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset + FutureRotationDirection * 100.f * DebugDrawScale,
					FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
				UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset,
					ComponentTransform.GetLocation() + DebugArrowOffset + FutureRotationDirection * 100.f * DebugDrawScale,
					FColor::Yellow.WithAlpha(DebugAlpha), TEXT(""));
			}
			*/

			const float ActualOrientationAngleDegrees = FMath::RadiansToDegrees(InstanceData->OrientationAngleForPoseWarpRad);
			const FVector WarpedRotationDirection = RotationDirection.RotateAngleAxis(ActualOrientationAngleDegrees, RotationAxisVector);

			DebugArrowOffset += FVector::ZAxisVector * DebugDrawScale;

			UE_VLOG_CIRCLE_THICK(InstanceData->HostObject, "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset + WarpedRotationDirection * 100.f * DebugDrawScale,
				FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
			UE_VLOG_ARROW(InstanceData->HostObject, "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + WarpedRotationDirection * 100.f * DebugDrawScale,
				FColor::Green.WithAlpha(DebugAlpha), TEXT(""));
		}
	}
#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
}

void FAnimNextStrafeWarpingTask::DoWarpPose(TUniquePtr<UE::UAF::FKeyframeState>& Keyframe, const FVector& RotationAxisVector) const
{
	using namespace UE::UAF;
	
	const FLODPoseStack& Pose = Keyframe->Pose;

	FTransformArrayView PoseTransforms = Pose.LocalTransformsView;
	if (PoseTransforms.IsEmpty())
	{
		// No bones
		return;
	}

	if (Pose.RefPose == nullptr)
	{
		// No ref pose, cannot continue
		return;
	}

	const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap = Pose.GetLODBoneIndexToParentLODBoneIndexMap();
	
	FTransformArraySoAStack ComponentSpaceTransforms(PoseTransforms.Num(), false);
	ComponentSpaceTransforms.CopyTransforms(PoseTransforms, 0, PoseTransforms.Num());
	ConvertPoseLocalToMeshRotationTranslation(ComponentSpaceTransforms.GetView(), LODBoneIndexToParentLODBoneIndexMap);

	const float RootOffset = FMath::UnwindRadians(InstanceData->OrientationAngleForPoseWarpRad * SharedData->DistributedBoneOrientationAlpha);

	const float IKFootRootOrientationAlpha = 1.f - SharedData->DistributedBoneOrientationAlpha;

	// Rotate IK Foot Root
	if (!FMath::IsNearlyZero(IKFootRootOrientationAlpha, KINDA_SMALL_NUMBER))
	{
		{
			const FQuat IKRootBoneRotation = FQuat(RotationAxisVector, InstanceData->OrientationAngleForPoseWarpRad * IKFootRootOrientationAlpha);
			// IK Feet 
			// We want these to keep their original component space orientation
			// But we want them to translate based on some rotation
			const int32 NumIKFootBones = SharedData->FootData.Num();

			if (NumIKFootBones > 0)
			{
				for (int32 ArrayIndex = 0; ArrayIndex < NumIKFootBones; ArrayIndex++)
				{
					const FStrafeWarpFootData& FootData = SharedData->FootData[ArrayIndex];
					FBoneIndexType LegTipIndex = Pose.FindLODBoneIndexFromBoneName(FootData.LegTip);
					if (LegTipIndex == INVALID_BONE_INDEX)
					{
						continue;
					}

					FTransform FootTargetTransform(ComponentSpaceTransforms.Rotations[LegTipIndex], ComponentSpaceTransforms.Translations[LegTipIndex]);
					FootTargetTransform.SetLocation( IKRootBoneRotation.RotateVector(FootTargetTransform.GetLocation()));

#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
					if (FVisualLogger::IsRecording())
					{
						const float DebugDrawSpehereRadius = 10.f;
						UE_VLOG_SPHERE(InstanceData->HostObject, "OrientationWarping", Display, InstanceData->RootBoneTransform.TransformPosition(FootTargetTransform.GetLocation()), DebugDrawSpehereRadius, FColor::Green, TEXT(""));
					}
#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG

					if (SharedData->FootMode == EStrafeWarpingFootMode::DoBasicIK)
					{
						FBoneIndexType LegRootIndex = Pose.FindLODBoneIndexFromBoneName(FootData.LegRoot);
						FBoneIndexType LegMidIndex = Pose.FindLODBoneIndexFromBoneName(FootData.LegMid);

						// Validate data
						if (LegRootIndex == INVALID_BONE_INDEX || LegMidIndex == INVALID_BONE_INDEX)
						{
							continue;
						}

						check(Pose.GetLODBoneParentIndex(LegTipIndex) == LegMidIndex);
						check(Pose.GetLODBoneParentIndex(LegMidIndex) == LegRootIndex);

						FTransform LegRootTransformCS(ComponentSpaceTransforms.Rotations[LegRootIndex], ComponentSpaceTransforms.Translations[LegRootIndex]);
						FTransform LegMidTransformCS(ComponentSpaceTransforms.Rotations[LegMidIndex], ComponentSpaceTransforms.Translations[LegMidIndex]);
						FTransform LegTipTransformCS(ComponentSpaceTransforms.Rotations[LegTipIndex], ComponentSpaceTransforms.Translations[LegTipIndex]);

						// Calculate pole vector
						// By using the side direction of the knee and the target, we can ensure the pole target is always 90 degrees from the target direction
						FVector PoleTarget = FootTargetTransform.GetLocation() - LegRootTransformCS.GetLocation();
						FVector SideVectorCS = LegMidTransformCS.GetRotation().RotateVector(FootData.JointSideVector);
						PoleTarget = PoleTarget.Cross(SideVectorCS);

						// Pole vector is expected to be provided as a location in AnimationCore::SolveTwoBoneIK
						// so convert from direction to location
						PoleTarget += LegRootTransformCS.GetLocation();

						AnimationCore::SolveTwoBoneIK(LegRootTransformCS,
							LegMidTransformCS,
							LegTipTransformCS,
							PoleTarget,
							FootTargetTransform.GetLocation(),
							false,
							0.0,
							0.0
							);

						LegTipTransformCS.SetRotation(FootTargetTransform.GetRotation());

						// Apply results

						// Be careful reverse TTransform vs FQuat order of operations
						FTransform LocalTip = LegTipTransformCS * LegMidTransformCS.Inverse();
						FTransform LocalMid = LegMidTransformCS * LegRootTransformCS.Inverse();

						// Figure out local root by applying difference in CS. We can't use the parent component space since it hasn't been updated (unlike the foot and knee)
						// This avoids computing all the component space transforms again
						// We can set the local transform of the root by applying the difference between the component space root transform
						FQuat LocalRootRotationDiff = ComponentSpaceTransforms[LegRootIndex].Rotation.Inverse() * LegRootTransformCS.GetRotation(); 

						PoseTransforms[LegTipIndex].Translation = LocalTip.GetLocation();
						PoseTransforms[LegTipIndex].Rotation = LocalTip.GetRotation();
						PoseTransforms[LegMidIndex].Translation = LocalMid.GetLocation();
						PoseTransforms[LegMidIndex].Rotation = LocalMid.GetRotation();
						PoseTransforms[LegRootIndex].Rotation = PoseTransforms[LegRootIndex].Rotation * LocalRootRotationDiff;
						

	#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
						if (FVisualLogger::IsRecording())
						{
							const float DebugDrawScale = 10.0f;
							const uint16 DebugDrawThickness = 1;
							const float DebugDrawSpehereRadius = 10.f;
							// Careful: FTransform multiplication is reveresed from quat!
							FTransform FootWorldTransformOriginal = FTransform(ComponentSpaceTransforms.Rotations[LegTipIndex], ComponentSpaceTransforms.Translations[LegTipIndex]) * InstanceData->RootBoneTransform;
							FTransform FootWorldTransformTarget = FTransform(ComponentSpaceTransforms.Rotations[LegTipIndex], FootTargetTransform.GetLocation()) * InstanceData->RootBoneTransform;
							FTransform FootWorldTransformSolved = LegTipTransformCS * InstanceData->RootBoneTransform;
							UE_VLOG_COORDINATESYSTEM(InstanceData->HostObject, "OrientationWarping", Display, FootWorldTransformTarget.GetLocation(), FootWorldTransformTarget.GetRotation().Rotator(), DebugDrawScale, FColor::Green, DebugDrawThickness, TEXT(""));
							UE_VLOG_COORDINATESYSTEM(InstanceData->HostObject, "OrientationWarping", Display, FootWorldTransformOriginal.GetLocation(), FootWorldTransformOriginal.GetRotation().Rotator(), DebugDrawScale, FColor::Green, DebugDrawThickness, TEXT(""));
							UE_VLOG_ARROW(InstanceData->HostObject, "OrientationWarping", Display, FootWorldTransformOriginal.GetLocation(), FootWorldTransformTarget.GetLocation(), FColor::Magenta, TEXT(""));
							UE_VLOG_ARROW(InstanceData->HostObject, "OrientationWarping", Display, LegMidTransformCS.GetLocation() + InstanceData->RootBoneTransform.GetLocation(), PoleTarget + InstanceData->RootBoneTransform.GetLocation(), FColor::Red, TEXT(""));
							UE_VLOG_SPHERE(InstanceData->HostObject, "OrientationWarping", Display, FootWorldTransformSolved.GetTranslation(), DebugDrawSpehereRadius, FColor::Yellow, TEXT(""));
						}
	#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
					}

					
					// Set the IK bone to match the foot target
					// Better to do this even if we also did basic IK, in case other nodes later down the chain rely on the ik target bone

					FBoneIndexType IKTargetIndex = Pose.FindLODBoneIndexFromBoneName(FootData.IKTargetBone);

					if (IKTargetIndex != INVALID_BONE_INDEX)
					{
						// Set IK target to equal leg tip in component space
						FBoneIndexType IKTargetParent = Pose.GetLODBoneParentIndex(IKTargetIndex);		// Usually the root bone but we can make it work for any parent
						if (IKTargetParent != INVALID_BONE_INDEX)
						{
							FTransform IKTargetCS = FootTargetTransform;
							FTransform IKTargetParentCS = ComponentSpaceTransforms[IKTargetParent];
							FTransform IKTargetLocal = IKTargetCS * IKTargetParentCS.Inverse();

							PoseTransforms[IKTargetIndex] = IKTargetLocal;
						}
					}
				
				}
			}
		}
	}
	
	// Rotate Root Bone first, as that cheaply rotates the whole pose with one transformation.
	// We do this with the pose in local space since we want it to propagate
	if (!FMath::IsNearlyZero(RootOffset, KINDA_SMALL_NUMBER))
	{
		if (SharedData->PreserveOriginalRootRotation)
		{
			// Find all children of the root and adjust them
			for (FBoneIndexType BoneIndex = 1; BoneIndex < Pose.GetNumBones(); BoneIndex++)
			{
				if (Pose.GetLODBoneParentIndex(BoneIndex) == 0)
				{
					// Is a child of the root
					const FVector LocalRotationVector = ComponentSpaceTransforms[BoneIndex].Rotation.UnrotateVector(RotationAxisVector);
					const FQuat RootRotation = FQuat(LocalRotationVector, RootOffset);
					PoseTransforms.Rotations[BoneIndex] = PoseTransforms.Rotations[BoneIndex] * RootRotation;
				}
			}
		}
		else
		{
			const FQuat RootRotation = FQuat(RotationAxisVector, RootOffset);

			PoseTransforms.Rotations[0] = PoseTransforms.Rotations[0] * RootRotation;
			PoseTransforms.Rotations[0].Normalize();
		}
	}

	const int32 NumSpineBones = SharedData->SpineBones.Num();
	const bool bSpineOrientationAlpha = !FMath::IsNearlyZero(SharedData->DistributedBoneOrientationAlpha, KINDA_SMALL_NUMBER);
	const bool bUpdateSpineBones = (NumSpineBones > 0) && bSpineOrientationAlpha;

	if (bUpdateSpineBones)
	{
		// Todo: Can we get away with lazy init here? Does the ref pose skeleton change at runtime?
		// Todo: Cache spine bone data
		TArray<FStrafeWarpingTrait::FSpineBoneData, TInlineAllocator<16>> SpineBoneDataArray;
		SpineBoneDataArray.SetNumUninitialized(NumSpineBones);
		FStrafeWarpingTrait::InitializeSpineData(TArrayView<FStrafeWarpingTrait::FSpineBoneData>(SpineBoneDataArray), SharedData->SpineBones, Pose);
		
		// Spine bones counter rotate body orientation evenly across all bones.
		// Note: reverse iteration is important! We go from child to parent
		for (int32 ArrayIndex = NumSpineBones - 1; ArrayIndex >= 0; ArrayIndex--)
		{
			const FStrafeWarpingTrait::FSpineBoneData& BoneData = SpineBoneDataArray[ArrayIndex];
			if (BoneData.LODBoneIndex == INVALID_BONE_INDEX || BoneData.Weight == 0.0f)
			{
				continue;
			}
			
			// Important note! The root was moved in local space, so our component transform array is actually out of date
			// However since we know everything rotated around RotationAxisVector, it doesn't matter for this calculation
			const FVector LocalRotationVector = ComponentSpaceTransforms[BoneData.LODBoneIndex].Rotation.UnrotateVector(RotationAxisVector);
			const FQuat SpineBoneCounterRotation = FQuat(LocalRotationVector, -InstanceData->OrientationAngleForPoseWarpRad * SharedData->DistributedBoneOrientationAlpha * BoneData.Weight);

			PoseTransforms.Rotations[BoneData.LODBoneIndex] = PoseTransforms.Rotations[BoneData.LODBoneIndex] * SpineBoneCounterRotation;
			PoseTransforms.Rotations[BoneData.LODBoneIndex].Normalize();
		}
	}
}

