// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "StrafeWarpingTrait.generated.h"

UENUM(BlueprintType)
enum class EStrafeWarpingFootMode : uint8
{
	DoBasicIK,
	SetIKBoneOnly,
};

// Small structure to define a simple IK setup for legs
USTRUCT()
struct FStrafeWarpFootData
{
	GENERATED_BODY()

	// Used for basic IK performed on the node itself
	// Only required if EStrafeWarpingFootMode is set to `DoBasicIK`
	// e.g. the thigh bone
	UPROPERTY(EditAnywhere, Category=Settings)
	FName LegRoot;

	// Used for basic IK performed on the node itself
	// Only required if EStrafeWarpingFootMode is set to `DoBasicIK`
	// e.g. the knee bone
	UPROPERTY(EditAnywhere, Category=Settings)
	FName LegMid;

	// This bone is required in all modes
	// e.g. the foot bone
	UPROPERTY(EditAnywhere, Category=Settings)
	FName LegTip;

	// Set this bone to the IK target (in character space)
	// Can be invalid
	UPROPERTY(EditAnywhere, Category=Settings)
	FName IKTargetBone;

	// Used for pole vector calculation. Should be the side of the knee in its local space
	UPROPERTY(EditAnywhere, Category=Settings)
	FVector JointSideVector = FVector::ZAxisVector;
};

USTRUCT(meta = (DisplayName = "Strafe Warping"))
struct FStrafeWarpingTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Current strength of the skeletal control
	UPROPERTY(EditAnywhere, Category = Alpha, meta = (PinShownByDefault))
	float Alpha = 1.0f;

	// The Orientation to steer towards
	UPROPERTY(EditAnywhere, Category=Evaluation, meta=(PinShownByDefault))
	FQuat TargetOrientation = FQuat::Identity;
	
	// @TODO Temp / try to remove this. Shouldn't have to feed as argument
	// Last root bone transform sampled
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (PinShownByDefault))
	FTransform RootBoneTransform = FTransform::Identity;

	// Rotation axis used when rotating the character body
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Inline))
	TEnumAsByte<EAxis::Type> RotationAxis = EAxis::Z;

	// Specifies how much rotation is applied to the character body versus IK feet
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Inline, ClampMin="0.0", ClampMax="1.0", PinHiddenByDefault))
	float DistributedBoneOrientationAlpha = 0.5f;

	// Specifies the interpolation speed (in Alpha per second) towards reaching the final warped rotation angle
	// A value of 0 will cause instantaneous rotation, while a greater value will introduce smoothing
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Inline, ClampMin="0.0"))
	float RotationInterpSpeed = 10.f;

	// Same as RotationInterpSpeed, but for CounterCompensate smoothing. A value of 0 sample raw root motion.
	// Used to avoid stuttering from resampling root deltas. Root motion is already smooth, so a large value is our default (~75% of 60 fps).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (Inline, ClampMin = "0.0"))
	float CounterCompensateInterpSpeed = 45.f;

	// Max correction we're allowed to do per-second when using interpolation.
	// This minimizes pops when we have a large difference between current and target orientation.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Inline, ClampMin="0.0", EditCondition="RotationInterpSpeed > 0.0f"))
	float MaxCorrectionDegrees = 180.f;

	// Don't compensate our interpolator when the instantaneous root motion delta is higher than this. This is likely a pivot.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Inline, ClampMin="0.0", EditCondition="RotationInterpSpeed > 0.0f"))
	float MaxRootMotionDeltaToCompensateDegrees = 45.f;

	// Whether to counter compensate interpolation by the animated root motion angle change over time.
	// This helps to conserve the motion from our animation.
	// Disable this if your root motion is expected to be jittery, and you want orientation warping to smooth it out.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Inline, EditCondition="RotationInterpSpeed > 0.0f"))
	bool bCounterCompenstateInterpolationByRootMotion = true;
	
	// Minimum root motion speed required to apply orientation warping
	// This is useful to prevent unnatural re-orientation when the animation has a portion with no root motion (i.e starts/stops/idles)
	// When this value is greater than 0, it's recommended to enable interpolation with RotationInterpSpeed > 0
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (ClampMin = "0.0", Inline))
	float MinRootMotionSpeedThreshold = 10.0f;
	
	// Specifies an angle threshold to prevent erroneous over-rotation of the character, disabled with a value of 0
	//
	// When the effective orientation warping angle is detected to be greater than this value (default: 90 degrees) the locomotion direction will be inverted prior to warping
	// This will be used in the following equation: [Orientation = RotationBetween(RootMotionDirection, -LocomotionDirection)]
	//
	// Example: Playing a forward running animation while the motion is going backward 
	// Rather than orientation warping by 180 degrees, the system will warp by 0 degrees 
	UPROPERTY(EditAnywhere, Category=Evaluation, meta=(Inline), meta=(ClampMin="0.0", ClampMax="180.0"))
	float LocomotionAngleDeltaThreshold = 90.f;

	// When true, propagates any modification on the root bone down to all of its children
	// When false, will directly modify the root
	UPROPERTY(EditAnywhere, Category=Evaluation, meta=(Inline))
	bool PreserveOriginalRootRotation = true;

	// Spine bone definitions
	// Used to counter rotate the body in order to keep the character facing forward
	// The amount of counter rotation applied is driven by DistributedBoneOrientationAlpha
	// TODO: we ideally would use an equivlane to FBoneReference
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Inline))
	TArray<FName> SpineBones;

	UPROPERTY(EditAnywhere, Category=Settings, meta=(DisplayName="IK Foot Bones"), meta=(Inline, Hidden))
	TArray<FStrafeWarpFootData> FootData;

	UPROPERTY(EditAnywhere, Category=Settings, meta=(Inline))
	EStrafeWarpingFootMode FootMode = EStrafeWarpingFootMode::DoBasicIK;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Alpha) \
		GeneratorMacro(TargetOrientation) \
		GeneratorMacro(RootBoneTransform) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FStrafeWarpingTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

/**
 * This trait modifies an animation pose to orientate towards a desired move direction 
 */
namespace UE::UAF
{

struct FStrafeWarpingTrait : FAdditiveTrait, IUpdate, IEvaluate
{
	DECLARE_ANIM_TRAIT(FStrafeWarpingTrait, FAdditiveTrait)

	using FSharedData = FStrafeWarpingTraitSharedData;

	// Internal structure with some precomputed weight values and bone indices
	struct FSpineBoneData
	{
		FBoneIndexType LODBoneIndex;
		float Weight;

		FSpineBoneData()
			: LODBoneIndex(INDEX_NONE)
			, Weight(0.f)
		{
		}

		FSpineBoneData(FBoneIndexType InBoneIndex)
			: LODBoneIndex(InBoneIndex)
			, Weight(0.f)
		{
		}

		// Comparison Operator for Sorting
		struct FCompareBoneIndex
		{
			FORCEINLINE bool operator()(const FSpineBoneData& A, const FSpineBoneData& B) const
			{
				return A.LODBoneIndex < B.LODBoneIndex;
			}
		};
	};

	struct FInstanceData : FTrait::FInstanceData
	{
		/** Target orientation per instance */
		FQuat TargetOrientation = FQuat::Identity;
		
		/** Last root bone transform sampled */
		FTransform RootBoneTransform = FTransform::Identity;

		/** Delta in seconds between updates, populated during PreUpdate */
		float DeltaTime = 0.f;

		float Alpha = 1.0f;

		// Internal current frame root motion delta direction
		FVector RootMotionDeltaDirection = FVector::ZeroVector;

		// Internal current frame root motion delta angle
		FQuat RootMotionDeltaRotation = FQuat::Identity;

		// Target for counter compenstate, we keep the target so we can smoothly interp.
		float CounterCompensateTargetAngleRad = 0.0f;

		// Internal orientation warping angle
		float OrientationAngleForPoseWarpRad = 0.f;
		
#if ENABLE_ANIM_DEBUG 
		/** Debug Object for VisualLogger */
		TObjectPtr<const UObject> HostObject = nullptr;
#endif // ENABLE_ANIM_DEBUG 
	};

	// IUpdate impl 
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

	// Helper for initialization of spine data
	// Ideally this would be called once on trait become relevent, but we don't know the reference pose until Execute right now so must call it every frame
	static void InitializeSpineData(TArrayView<FSpineBoneData> OutSpineBoneData, const TArray<FName>& SpineBoneNames, const FLODPoseStack& Pose);
};

} // namespace UE::UAF

/** Task to run Strafe Warping on VM */
USTRUCT()
struct FAnimNextStrafeWarpingTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextStrafeWarpingTask)

	static FAnimNextStrafeWarpingTask Make(UE::UAF::FStrafeWarpingTrait::FInstanceData* InstanceData, const UE::UAF::FStrafeWarpingTrait::FSharedData* SharedData);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

private:
	void DoWarpRootMotion(TUniquePtr<UE::UAF::FKeyframeState>& KeyframeState, const FVector& RotationAxisVector, const FVector& TargetMoveDir, float& OutTargetOrientationAngleRad) const;
	void DoWarpPose(TUniquePtr<UE::UAF::FKeyframeState>& KeyframeState, const FVector& RotationAxisVector) const;

	UE::UAF::FStrafeWarpingTrait::FInstanceData* InstanceData = nullptr;
	const UE::UAF::FStrafeWarpingTrait::FSharedData* SharedData = nullptr;
};