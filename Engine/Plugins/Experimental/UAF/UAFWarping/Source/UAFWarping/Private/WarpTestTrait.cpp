// Copyright Epic Games, Inc. All Rights Reserved.

#include "WarpTestTrait.h"
#include "Animation/AnimRootMotionProvider.h"
#include "AnimNextWarpingLog.h"
#include "Component/AnimNextComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "EvaluationVM/EvaluationVM.h"
#include "Module/AnimNextModuleInstance.h"
#include "VisualLogger/VisualLogger.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FWarpTestTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FWarpTestTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)

	#undef TRAIT_INTERFACE_ENUMERATOR

	void FWarpTestTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::PreUpdate(Context, Binding, TraitState);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		check(InstanceData);

		const float SecondsToWait = SharedData->GetSecondsToWait(Binding);
		const TArray<FTransform>& Transforms = SharedData->GetTransforms(Binding);

		if (Transforms.IsEmpty())
		{
			InstanceData->CurrentTransformIndex = 0;
			InstanceData->CurrentTime = 0.f;
		}
		else
		{
			if (InstanceData->CurrentTransformIndex >= Transforms.Num())
			{
				InstanceData->CurrentTransformIndex = 0;
			}

			InstanceData->CurrentTime += TraitState.GetDeltaTime();
			if (InstanceData->CurrentTime > SecondsToWait)
			{
				InstanceData->CurrentTime -= SecondsToWait;
				InstanceData->CurrentTransformIndex = (InstanceData->CurrentTransformIndex + 1) % Transforms.Num();
			}
		}
	}

	void FWarpTestTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		// @todo: WIP hacky, non thread safe (unless proper tick dependencies are in place) way to retrieve the mesh transform until we find a better way
		if (const FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
		{
			if (const UUAFComponent* AnimNextComponent = Cast<UUAFComponent>(ModuleInstance->GetObject()))
			{
				const AActor* Actor = AnimNextComponent->GetOwner();
				check(Actor);
				if (const USkeletalMeshComponent* SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>())
				{
					const TArray<FTransform>& Transforms = SharedData->GetTransforms(Binding);
					
					FAnimNextWarpTestTask Task;
					Task.ComponentTransform = SkeletalMeshComponent->GetComponentTransform();
					Task.WarpToTransform = Transforms[InstanceData->CurrentTransformIndex];
		
#if ENABLE_ANIM_DEBUG 
					Task.HostObject = Context.GetHostObject();
#endif //ENABLE_ANIM_DEBUG 
					Context.AppendTask(Task);
				}
			}
		}		
	}
} // namespace UE::UAF

void FAnimNextWarpTestTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
	if (!RootMotionProvider)
	{
		UE_LOGF(LogAnimNextWarping, Error, "FAnimNextWarpTestTask::Execute, missing RootMotionProvider");
	}
	else if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
	{
		// snippet on how to extract root motion
		// FTransform ThisFrameRootMotionTransform = FTransform::Identity;
		// if (RootMotionProvider->ExtractRootMotion(Keyframe->Get()->Attributes, ThisFrameRootMotionTransform))
		
#if ENABLE_ANIM_DEBUG 
		static const TCHAR* LogName = TEXT("WarpTest");
		UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, WarpToTransform.GetLocation(), WarpToTransform.GetRotation().GetAxisX() * 100.f + WarpToTransform.GetLocation(), FColor::Red, 1, TEXT(""));
		UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, WarpToTransform.GetLocation(), WarpToTransform.GetRotation().GetAxisY() * 100.f + WarpToTransform.GetLocation(), FColor::Blue, 1, TEXT(""));

		UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, ComponentTransform.GetLocation(), ComponentTransform.GetRotation().GetAxisX() * 80.f + ComponentTransform.GetLocation(), FColor::Black, 1, TEXT(""));
		UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, ComponentTransform.GetLocation(), ComponentTransform.GetRotation().GetAxisY() * 80.f + ComponentTransform.GetLocation(), FColor::Green, 1, TEXT(""));
#endif // ENABLE_ANIM_DEBUG 

		const FTransform RootMotion = WarpToTransform.GetRelativeTransform(ComponentTransform);

		RootMotionProvider->OverrideRootMotion(RootMotion, Keyframe->Get()->Attributes);
	}
}
