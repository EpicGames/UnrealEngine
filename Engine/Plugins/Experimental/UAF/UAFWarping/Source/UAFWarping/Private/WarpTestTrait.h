// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "WarpTestTrait.generated.h"

USTRUCT(Experimental, meta = (DisplayName = "WarpTest"))
struct FWarpTestTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// the trait will warp the character looping between Transforms[i] choosing the next one every SecondsToWait
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (PinShownByDefault))
	TArray<FTransform> Transforms;

	// every SecondsToWait we warp to the next Transforms[i]
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (PinShownByDefault))
	float SecondsToWait = 1.f;

	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Transforms) \
		GeneratorMacro(SecondsToWait) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FWarpTestTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{

struct FWarpTestTrait : FAdditiveTrait, IUpdate, IEvaluate
{
	DECLARE_ANIM_TRAIT(FWarpTestTrait, FAdditiveTrait)

	using FSharedData = FWarpTestTraitSharedData;

	struct FInstanceData : FTrait::FInstanceData
	{
		int32 CurrentTransformIndex = 0;
		float CurrentTime = 0.f;
	};

	// IUpdate impl 
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
};

} // namespace UE::UAF

USTRUCT(Experimental)
struct FAnimNextWarpTestTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextWarpTestTask)

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	FTransform ComponentTransform = FTransform::Identity;
	FTransform WarpToTransform = FTransform::Identity;

#if ENABLE_ANIM_DEBUG
	// Debug Object for VisualLogger
	const UObject* HostObject = nullptr;
#endif // ENABLE_ANIM_DEBUG
};