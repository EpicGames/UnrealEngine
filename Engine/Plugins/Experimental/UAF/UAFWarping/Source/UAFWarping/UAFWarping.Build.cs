// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFWarping : ModuleRules
	{
		public UAFWarping(ReadOnlyTargetRules Target) : base(Target)
		{
			bAllowUETypesInNamespaces = true;
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"AnimationCore",
					"RigVM",
					"Engine",
					"UAF",
					"UAFAnimGraph",
					"UAFAnimNode",
				}
			);
		}
	}
}