{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "UAF Warping",
	"Description": "Framework for animation and pose warping for UAF.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
		{
			"Name": "UAFAnimNode",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "UAFWarping",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	]
}
